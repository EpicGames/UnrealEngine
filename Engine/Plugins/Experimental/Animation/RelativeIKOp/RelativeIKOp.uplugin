{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Spatially Aware Retarget Ops",
	"Description": "A collection of Retarget Ops for preserving spatial relationships on retargeted animations",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "RelativeIKOp",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RelativeBodyAnimInfo",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "BodyIntersectIKOp",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RelativeBodyAnimUtils",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PreviewPropOp",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
	],
	"Plugins": [
		{
			"Name": "IKRig",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		}
	]
}