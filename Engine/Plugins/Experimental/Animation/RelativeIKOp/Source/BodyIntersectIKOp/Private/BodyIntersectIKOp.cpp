// Copyright Epic Games, Inc. All Rights Reserved.

#include "BodyIntersectIKOp.h"

#include "IKRigObjectVersion.h"
#include "PhysicsBodyHelpers.h"
#include "Engine/SkeletalMesh.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"
#include "Rig/Solvers/PointsToRotation.h"

#if WITH_EDITOR
#include "PrimitiveDrawingUtils.h"
#include "Engine/EngineTypes.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(BodyIntersectIKOp)

#define LOCTEXT_NAMESPACE "BodyIntersectIKOp"


const UClass* FIKRetargetBodyIntersectIKOpSettings::GetControllerType() const
{
	return UIKRetargetBodyIntersectController::StaticClass();
}

void FIKRetargetBodyIntersectIKOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	static TArray<FName> PropertiesToIgnore = {};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetBodyIntersectIKOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

bool FIKRetargetBodyIntersectIKOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;

	// this op requires a parent to supply an IK Rig
	if (!ensure(InParentOp))
	{
		return false;
	}

	// validate that an IK rig has been assigned
	const FIKRetargetRunIKRigOp* ParentRigOp = reinterpret_cast<const FIKRetargetRunIKRigOp*>(InParentOp);
	if (!ParentRigOp || ParentRigOp->Settings.IKRigAsset == nullptr)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingIKRig", "{0}, is missing an IK rig. No chains can be retargeted. "), FText::FromName(GetName())));
		return false;
	}

	PhysicsAsset = Settings.TargetPhysicsAssetOverride;
	if (!PhysicsAsset)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingPhysicsAsset", "{0}: Intersect Physics Asset must be specified. "), FText::FromName(GetName())));
		return false;
	}
	
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	const TArray<FResolvedBoneChain>* TargetChains = BoneChains.GetResolvedBoneChains(ERetargetSourceOrTarget::Target, ParentRigOp->Settings.IKRigAsset);
	if (!TargetChains)
	{
		return false;
	}
	
	InitPoleVectorIntersectors(*TargetChains, InTargetSkeleton);
	
	bIsInitialized = true;
	return bIsInitialized;
}

void FIKRetargetBodyIntersectIKOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
#if WITH_EDITOR
	ResetDebugInfo();
#endif //WITH_EDITOR
	
	if (InProcessor.IsIKForcedOff())
	{
		return;
	}

	if (!bIsInitialized)
	{
		return;
	}

	FDebugBodyIntersectDrawInfo LocalDebugInfo;
#if WITH_EDITOR
	LocalDebugInfo.TargetIntersectDetected.SetNumZeroed(Settings.IntersectBodies.Num());
#endif //WITH_EDITOR
	
	// Clear move info every frame (setup in init instead?)
	MoveWithPropGroups.Init({},Settings.PropIntersectSettings.Num());

	// Store deltas for moving together (max -> blend)
	FrameDeltaInfo.Init({}, Settings.IntersectGoalSettings.Num());
	
	// Clear update prop locations
	UpdatePropDeltas.Reset(Settings.PropIntersectSettings.Num());
	
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (int i = 0; i < Settings.IntersectGoalSettings.Num(); i++)
	{
		FIKGoalIntersectShapeSettings& GoalSettings = Settings.IntersectGoalSettings[i];
		FIKRigGoal* IKGoal = GoalContainer.FindGoalByName(GoalSettings.Goal);
		if (!IKGoal || !IKGoal->bEnabled)
		{
			continue;
		}

		const FName GoalBoneName = IKGoal->BoneName;
		int32 TargetBoneIdx = InProcessor.GetTargetSkeleton().FindBoneIndexByName(GoalBoneName);
		if (TargetBoneIdx == INDEX_NONE)
		{
			continue;
		}
		
		const FTransform TargetBoneTfm = OutTargetGlobalPose[TargetBoneIdx];
		UpdatePropMoves(GoalSettings.PropIntersectBone, i);

		int32 CheckCollisionIdx = PhysicsAsset->FindBodyIndex(IKGoal->BoneName);
		if (CheckCollisionIdx == INDEX_NONE)
		{
			continue;
		}
		
		const FKShapeElem* GoalShape = FPhysShapeUtils::FindBodyShape(PhysicsAsset, IKGoal->BoneName);
		double GoalRadius = FPhysShapeUtils::CalcShapeSmallRadius(GoalShape); // * GoalSettings.SphereScale;
		if (FMath::IsNearlyZero(GoalRadius))
		{
			continue;
		}
		
		FVector CompGoalUpdate = GoalLocBlendCompSpace(IKGoal, TargetBoneTfm);
		FVector CompGoalShapeOffset = TargetBoneTfm.TransformVector(GoalSettings.GoalShapeOffset);
		FVector CompGoalShapeLoc = CompGoalUpdate + CompGoalShapeOffset;
		FTransform CompShapeTfm = FTransform(TargetBoneTfm.Rotator(), CompGoalShapeLoc, TargetBoneTfm.GetScale3D()*GoalSettings.GoalShapeScale);
		
		FVector GoalDelta = GetTotalIntersectDelta(CheckCollisionIdx, GoalShape, CompShapeTfm, OutTargetGlobalPose, InProcessor.GetTargetSkeleton(), LocalDebugInfo);
		FrameDeltaInfo[i] = {IKGoal,CompShapeTfm,CompGoalUpdate,GoalDelta};
	}
	
	for (int PropIdx = 0; PropIdx < Settings.PropIntersectSettings.Num(); ++PropIdx)
	{
		const FIKPropIntersectSettings& PropSettings = Settings.PropIntersectSettings[PropIdx];
		if (PropSettings.BoneName == NAME_None)
		{
			continue;
		}
		
		if (!PropSettings.bPinBone && MoveWithPropGroups[PropIdx].IsEmpty())
		{
			continue;
		}
		
		// Get prop intersection-delta
		FVector PropDelta = FVector::ZeroVector;
		if (!FMath::IsNearlyZero(PropSettings.CapsuleRadius))
		{
			int32 PropBoneIdx = InProcessor.GetTargetSkeleton().FindBoneIndexByName(PropSettings.BoneName);
			if (PropBoneIdx == INDEX_NONE)
			{
				continue;
			}
			
			FKSphylElem PropShape = FKSphylElem(PropSettings.CapsuleRadius, PropSettings.CapsuleLength);
			FTransform PropTfm = FTransform(PropSettings.CapsuleRotation, PropSettings.CapsuleTranslation) * OutTargetGlobalPose[PropBoneIdx];
			
			PropDelta = GetTotalIntersectDelta(INDEX_NONE, &PropShape, PropTfm, OutTargetGlobalPose, InProcessor.GetTargetSkeleton(), LocalDebugInfo);
		}
		
		// Nothing else to mess with if no linked goals
		if (MoveWithPropGroups[PropIdx].IsEmpty())
		{
			continue;
		}
		
		// Get consensus delta
		// TODO: Use delta along avg for all bodies?
		for (int32 GoalIdx : MoveWithPropGroups[PropIdx])
		{
			// const FVector& GoalDelta = FrameDeltaInfo[GoalIdx].IntersectDelta;
			// if (GoalDelta.SquaredLength() > PropDelta.SquaredLength())
			// {
			// 	PropDelta = GoalDelta;
			// }
			
			// TODO: Weight starting deltas for each intersect, instead of at end?
			FIKRigGoal* IKGoal = FrameDeltaInfo[GoalIdx].GoalPtr;
			if (!IKGoal)
			{
				continue;
			}
			
			int32 CheckCollisionIdx = PhysicsAsset->FindBodyIndex(IKGoal->BoneName);
			if (CheckCollisionIdx == INDEX_NONE)
			{
				continue;
			}
			
			const FKShapeElem* GoalShape = FPhysShapeUtils::FindBodyShape(PhysicsAsset, IKGoal->BoneName);
			double GoalRadius = FPhysShapeUtils::CalcShapeSmallRadius(GoalShape); // * GoalSettings.SphereScale;
			if (FMath::IsNearlyZero(GoalRadius))
			{
				continue;
			}
			
			const FVector StartLoc = FrameDeltaInfo[GoalIdx].StartGoalLoc;
			FTransform GoalShapeTfm =  FrameDeltaInfo[GoalIdx].StartBodyTfm;
			GoalShapeTfm.SetLocation(StartLoc + PropDelta);
			
			PropDelta += GetTotalIntersectDelta(CheckCollisionIdx, GoalShape, GoalShapeTfm, OutTargetGlobalPose, InProcessor.GetTargetSkeleton(), LocalDebugInfo);
		}
		
		// Propagate max delta back (with blend) to goal offsets
		for (int32 GoalIdx : MoveWithPropGroups[PropIdx])
		{
			double MoveWithAlpha = Settings.IntersectGoalSettings[GoalIdx].BlendMultiMaxOffset;
			FVector& GoalDelta = FrameDeltaInfo[GoalIdx].IntersectDelta;
			GoalDelta = FMath::Lerp(GoalDelta, PropDelta, MoveWithAlpha);
		}
		
		// Setup post-solve update for prop bones, if we are pinning
		if (PropSettings.bPinBone)
		{
			int32 PropBoneIdx = InProcessor.GetTargetSkeleton().FindBoneIndexByName(PropSettings.BoneName);
			if (PropBoneIdx == INDEX_NONE)
			{
				continue;
			}
			
			UpdatePropDeltas.Add({PropBoneIdx, PropDelta});
		}
		
#if WITH_EDITOR
		if (Settings.bDebugDraw && !FMath::IsNearlyZero(PropSettings.CapsuleRadius))
		{
			int32 PropBoneIdx = InProcessor.GetTargetSkeleton().FindBoneIndexByName(PropSettings.BoneName);
			if (PropBoneIdx == INDEX_NONE)
			{
				continue;
			}
			
			FTransform PropTfm = FTransform(PropSettings.CapsuleRotation, PropSettings.CapsuleTranslation) * OutTargetGlobalPose[PropBoneIdx];
			PropTfm.SetTranslation(PropTfm.GetTranslation() + PropDelta);
			LocalDebugInfo.PropIntersectTfms.Emplace(PropSettings.CapsuleRadius,PropSettings.CapsuleLength, PropTfm);
		}
#endif //WITH_EDITOR
	}
	
	for (int i = 0; i < FrameDeltaInfo.Num(); i++)
	{
		FIKRigGoal* IKGoal = FrameDeltaInfo[i].GoalPtr;
		if (!IKGoal)
		{
			continue;
		}
		
		FTransform DebugShapeTfm = FrameDeltaInfo[i].StartBodyTfm;
		const FVector& GoalBaseLoc = FrameDeltaInfo[i].StartGoalLoc;
		const FVector& GoalDelta = FrameDeltaInfo[i].IntersectDelta;
		IKGoal->Position = GoalBaseLoc + GoalDelta;
		IKGoal->PositionAlpha = 1.0f;
		// SetGoalPosFromCompSpace(IKGoal, TargetBoneTfm, CompGoalUpdate + GoalDelta);
#if WITH_EDITOR
		if (Settings.bDebugDraw)
		{
			DebugShapeTfm.SetTranslation(GoalBaseLoc + GoalDelta);
			LocalDebugInfo.GoalIntersectTfms.Emplace(IKGoal->BoneName, DebugShapeTfm);
		}
#endif //WITH_EDITOR
	}

#if WITH_EDITOR
	if (Settings.bDebugDraw)
	{
		FScopeLock ScopeLock(&DebugDataMutex);
		
		DebugDrawInfo.GoalIntersectTfms = LocalDebugInfo.GoalIntersectTfms;
		DebugDrawInfo.PropIntersectTfms = LocalDebugInfo.PropIntersectTfms;

		// Cache debug info for target intersect bodies
		DebugDrawInfo.TargetIntersectTfms.Reset(Settings.IntersectBodies.Num());
		DebugDrawInfo.TargetIntersectDetected = LocalDebugInfo.TargetIntersectDetected;
		for (FName TargetBoneName : Settings.IntersectBodies)
		{
			int32 TargetPoseIdx = InProcessor.GetTargetSkeleton().FindBoneIndexByName(TargetBoneName);
			if (TargetPoseIdx == INDEX_NONE)
			{
				continue;
			}
			
			DebugDrawInfo.TargetIntersectTfms.Add({TargetBoneName,OutTargetGlobalPose[TargetPoseIdx]});
		}
	}
#endif
}

void FIKRetargetBodyIntersectIKOp::RunAfterParent(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	if (InProcessor.IsIKForcedOff())
	{
		return;
	}

	if (!bIsInitialized)
	{
		return;
	}
	
	// Apply prop offsets if appropriate
	for (const TPair<int32,FVector>& PropPinDelta : UpdatePropDeltas)
	{
		FTransform& AttachTfm = OutTargetGlobalPose[PropPinDelta.Key];
		AttachTfm.SetTranslation(AttachTfm.GetTranslation() + PropPinDelta.Value);
	}
	
	if (!Settings.bEnablePoleVectorIntersect)
	{
		return;
	}
	
	FDebugBodyIntersectDrawInfo LocalDebugInfo;
	
	const FRetargetSkeleton& TargetSkeleton = InProcessor.GetTargetSkeleton();
	for (const FPoleVectorChainIntersector& PVI : PoleVectorChainIntersectors)
	{
		PVI.ApplyIntersectRotation(TargetSkeleton, OutTargetGlobalPose, LocalDebugInfo);
	}
	
#if WITH_EDITOR
	if (Settings.bDebugDraw)
	{
		FScopeLock ScopeLock(&DebugDataMutex);
		DebugDrawInfo.PoleIntersectTfms = LocalDebugInfo.PoleIntersectTfms;
	}
#endif //WITH_EDITOR
}

void FPoleVectorChainIntersector::Initialize(
	const FIKRetargetBodyIntersectIKOpSettings* InSettings,
	const TArray<int32>& InBodySettingsIdx,
	const FResolvedBoneChain* InTargetChain,
	const FTargetSkeleton& InTargetSkeleton)
{
	Settings = InSettings;
	PhysicsAsset = InSettings->TargetPhysicsAssetOverride;
	TargetBoneChain = InTargetChain;
	IntersectBonesIdx.Reset();
	PoleBodiesSettingsIdx.Reset();
	PoleBonesIdx.Reset();
	AllChildrenWithinChain.Reset();
	
	for (int32 SettingsIdx : InBodySettingsIdx)
	{
		const FIKPoleVectorIntersectSettings& PvIsectSetting = Settings->PoleVectorIntersectSettings[SettingsIdx];
		int32 BoneIdx = InTargetSkeleton.FindBoneIndexByName(PvIsectSetting.BodyName);
		if (BoneIdx == INDEX_NONE)
		{
			continue;
		}
		
		PoleBodiesSettingsIdx.Add(SettingsIdx);
		PoleBonesIdx.Add(BoneIdx);
	}
	
	for (const FName IntersectBone : Settings->IntersectBodies)
	{
		// TODO: Normalize code to remove FName->Index maps and just keep struct of bones/indices
		int32 BoneIdx = InTargetSkeleton.FindBoneIndexByName(IntersectBone);
		IntersectBonesIdx.Add(BoneIdx);
	}
	
	// cache indices of bones in chain (and their children recursively) excluding children of the end bone
	TArray<int32> AllChildrenIndices;
	InTargetSkeleton.GetChildrenIndicesRecursive(TargetBoneChain->BoneIndices[0],AllChildrenIndices);
	TArray<int32> ChildrenOfEndIndices = {TargetBoneChain->BoneIndices.Last()};
	InTargetSkeleton.GetChildrenIndicesRecursive(TargetBoneChain->BoneIndices.Last(),ChildrenOfEndIndices);
	
	for (int32 ChildIndex : AllChildrenIndices)
	{
		if (ChildrenOfEndIndices.Contains(ChildIndex))
		{
			continue;
		}
		AllChildrenWithinChain.Add(ChildIndex);
	}
}

void FPoleVectorChainIntersector::ApplyIntersectRotation(
	const FRetargetSkeleton& TargetSkeleton,
	TArray<FTransform>& OutTargetGlobalPose,
	FDebugBodyIntersectDrawInfo& DebugInfo) const
{
	const FVector TargetChainAxisNorm = GetChainAxisNormalized(TargetBoneChain->BoneIndices, OutTargetGlobalPose);
	const FVector ChainOriginLoc = OutTargetGlobalPose[TargetBoneChain->BoneIndices[0]].GetLocation();
	
	// Get all child local bone transforms
	TArray<FTransform> AllChildrenInputLocalTransforms = TargetSkeleton.GetLocalTransformsOfMultipleBones(AllChildrenWithinChain, OutTargetGlobalPose);
	
	double TotalWeight = 0.0;
	double TotalAngle = 0.0;
	for (int32 Index = 0; Index < PoleBonesIdx.Num(); ++Index)
	{
		int32 BoneIdx = PoleBonesIdx[Index];
		int32 SettingsIdx = PoleBodiesSettingsIdx[Index];
		
		if (!Settings->PoleVectorIntersectSettings.IsValidIndex(SettingsIdx))
		{
			continue;
		}
		
		const FTransform TargetBoneTfm = OutTargetGlobalPose[BoneIdx];
		
		// TODO: Should we copy these settings into the struct?
		const FIKPoleVectorIntersectSettings& BodySettings = Settings->PoleVectorIntersectSettings[SettingsIdx];

		const FKShapeElem* BodyShape = FPhysShapeUtils::FindBodyShape(PhysicsAsset, BodySettings.BodyName);
		double BodyRadius = FPhysShapeUtils::CalcShapeSmallRadius(BodyShape) * BodySettings.SphereScale;
		if (FMath::IsNearlyZero(BodyRadius))
		{
			continue;
		}
		
		FTransform LocalTfm(BodySettings.SphereOffset);
		FTransform PoleTfm = LocalTfm * TargetBoneTfm;
		FVector SphereLoc = PoleTfm.GetLocation();
		FVector SphereOriginVec = SphereLoc - ChainOriginLoc;
		FVector SpherePoleVec = FVector::VectorPlaneProject(SphereOriginVec, TargetChainAxisNorm);
		
		// TODO: Potentially blend/early-out when pole vec is much smaller than sphere radius
		
		FKSphereElem PoleBodySphere;
		PoleBodySphere.Radius = static_cast<float>(BodyRadius);
		PoleBodySphere.Center = BodySettings.SphereOffset;
		
		FVector IsectDelta = GetIntersectDelta(PoleTfm, &PoleBodySphere, OutTargetGlobalPose, DebugInfo);
		FVector PoleDelta = FVector::VectorPlaneProject(IsectDelta, TargetChainAxisNorm);
		if (PoleDelta.IsNearlyZero())
		{
			continue;
		}
		
		FVector DesiredLoc = SpherePoleVec + PoleDelta;
		
		FVector PoleNorm = SpherePoleVec.GetSafeNormal();
		FVector RotDir = FVector::CrossProduct(TargetChainAxisNorm, PoleNorm);
		
		FVector DesiredNorm = DesiredLoc.GetSafeNormal();
		double Angle = FMath::Atan2(FVector::DotProduct(DesiredNorm, RotDir), FVector::DotProduct(DesiredNorm,PoleNorm));
		
		TotalAngle += Angle;
		TotalWeight += 1.0;
	}
	
	if (!FMath::IsNearlyZero(TotalWeight))
	{
		FQuat Rotation(TargetChainAxisNorm, TotalAngle / TotalWeight);
	
		// rotate the base of the chain to match the pole vectors
		FTransform& BaseOfChain = OutTargetGlobalPose[TargetBoneChain->BoneIndices[0]];
		BaseOfChain.SetRotation(Rotation * BaseOfChain.GetRotation());
	
		// Update changes to children of the chain
		TargetSkeleton.UpdateGlobalTransformsOfMultipleBones(AllChildrenWithinChain, AllChildrenInputLocalTransforms,OutTargetGlobalPose);
	}
	
#if WITH_EDITOR
	if (Settings->bDebugDraw)
	{
		for (int32 Index = 0; Index < PoleBonesIdx.Num(); ++Index)
		{
			int32 BoneIdx = PoleBonesIdx[Index];
			int32 SettingsIdx = PoleBodiesSettingsIdx[Index];
		
			if (!Settings->PoleVectorIntersectSettings.IsValidIndex(SettingsIdx))
			{
				continue;
			}
		
			const FTransform TargetBoneTfm = OutTargetGlobalPose[BoneIdx];
			
			const FIKPoleVectorIntersectSettings& BodySettings = Settings->PoleVectorIntersectSettings[SettingsIdx];

			const FKShapeElem* BodyShape = FPhysShapeUtils::FindBodyShape(PhysicsAsset, BodySettings.BodyName);
			double BodyRadius = FPhysShapeUtils::CalcShapeSmallRadius(BodyShape) * BodySettings.SphereScale;
			if (FMath::IsNearlyZero(BodyRadius))
			{
				continue;
			}
		
			FTransform LocalTfm(BodySettings.SphereOffset);
			DebugInfo.PoleIntersectTfms.Emplace(static_cast<float>(BodyRadius), LocalTfm * TargetBoneTfm);
		}
	}
#endif //WITH_EDITOR
}

FVector FPoleVectorChainIntersector::GetIntersectDelta(
	const FTransform& PoleTfm,
	FKSphereElem* PoleSphere,
	const TArray<FTransform>& TargetGlobalPose,
	FDebugBodyIntersectDrawInfo& DebugInfo) const
{
	FVector TotalDelta = FVector::ZeroVector;
	for (int Index = 0; Index < Settings->IntersectBodies.Num(); ++Index)
	{
		if ( !IntersectBonesIdx.IsValidIndex(Index) )
		{
			continue;
		}
		
		int32 BoneIdx = IntersectBonesIdx[Index];
		if (BoneIdx == INDEX_NONE)
		{
			continue;
		}
		
		FName BodyName = Settings->IntersectBodies[Index];
		if (PhysicsAsset->FindBodyIndex(BodyName) == INDEX_NONE)
		{
			continue;
		}
		
		FTransform BodyTfm = TargetGlobalPose[BoneIdx];
		const FKShapeElem* ShapeElem = FPhysShapeUtils::FindBodyShape(PhysicsAsset, BodyName);
		if (!ShapeElem)
		{
			continue;
		}

		FVector DeltaDir = FVector::ZeroVector;
		double Dist = FBodyIntersectUtils::CalcIntersectionPairDelta(PoleTfm, PoleSphere, BodyTfm, ShapeElem, DeltaDir);
		if (Dist > 0.0)
		{
			TotalDelta += DeltaDir * Dist;
#if WITH_EDITOR
			// TODO: Add intersection detected code here
#endif //WITH_EDITOR
		}
	}
	
	return TotalDelta;
}

FVector FPoleVectorChainIntersector::CalculatePoleVector(
	const EAxis::Type& PoleAxis,
	const TArray<int32>& BoneIndices,
	const TArray<FTransform>& GlobalPose) const
{
	check(!BoneIndices.IsEmpty())

	const FVector ChainNormal = GetChainAxisNormalized(BoneIndices, GlobalPose);
	const FVector UnitPoleAxis = GlobalPose[BoneIndices[0]].GetUnitAxis(PoleAxis);
	const FVector PoleVector = FVector::VectorPlaneProject(UnitPoleAxis, ChainNormal);
	return PoleVector.GetSafeNormal();
}

FVector FPoleVectorChainIntersector::GetChainAxisNormalized(
	const TArray<int32>& BoneIndices,
	const TArray<FTransform>& GlobalPose) const
{
	const FVector ChainOrigin = GlobalPose[BoneIndices[0]].GetLocation();
	const FVector ChainAxis = GlobalPose[BoneIndices.Last()].GetLocation() - ChainOrigin;
	return ChainAxis.GetSafeNormal();
}

FVector FIKRetargetBodyIntersectIKOp::GoalLocBlendCompSpace(const FIKRigGoal* Goal, const FTransform& BoneTfm) const
{
	switch (Goal->PositionSpace)
	{
	case EIKRigGoalSpace::Additive:
		{
			return FMath::Lerp(FVector::ZeroVector, Goal->Position, Goal->PositionAlpha) + BoneTfm.GetLocation();
		}
	case EIKRigGoalSpace::Component:
		{
			return FMath::Lerp(BoneTfm.GetLocation(), Goal->Position, Goal->PositionAlpha);
		}
	case EIKRigGoalSpace::World:
		{
			// We assume no World-space goals will be set using retarget stack
		}
	}

	return BoneTfm.GetLocation();
}

void FIKRetargetBodyIntersectIKOp::SetGoalPosFromCompSpace(FIKRigGoal* Goal, const FTransform& BoneTfm, const FVector& CompSpaceLoc) const
{
	// TODO: Alpha passthrough and/or alpha ignore
	// Should we just update position space to be component always?
	switch (Goal->PositionSpace)
	{
	case EIKRigGoalSpace::Additive:
		{
			Goal->Position = CompSpaceLoc - BoneTfm.GetLocation();
			Goal->PositionAlpha = 1.0f;
			break;
		}
	case EIKRigGoalSpace::Component:
		{
			Goal->Position = CompSpaceLoc;
			Goal->PositionAlpha = 1.0f;
			break;
		}
	case EIKRigGoalSpace::World:
	default:
		{
			// We assume no World-space goals will be set using retarget stack
			Goal->Position = CompSpaceLoc;
			Goal->PositionAlpha = 1.0f;
			break;
		}
	}
}

void FIKRetargetBodyIntersectIKOp::InitPoleVectorIntersectors(
	const TArray<FResolvedBoneChain>& TargetChains,
	const FTargetSkeleton& InTargetSkeleton)
{
	PoleVectorChainIntersectors.Reset();
	
	if (!Settings.bEnablePoleVectorIntersect)
	{
		return;
	}
	
	// TODO: Can remove a lot of the checks/nesting in the loop if we use fix up code to user per-chain settings/info instead
	TMap<const FResolvedBoneChain*, TArray<int32>> ChainSetupMap;
	for (int32 SettingsIdx = 0; SettingsIdx < Settings.PoleVectorIntersectSettings.Num(); ++SettingsIdx)
	{
		const FIKPoleVectorIntersectSettings& PoleVecInfo = Settings.PoleVectorIntersectSettings[SettingsIdx];
		int32 BoneIdx = InTargetSkeleton.FindBoneIndexByName(PoleVecInfo.BodyName);
		if (BoneIdx == INDEX_NONE)
		{
			continue;
		}
		
		for (const FResolvedBoneChain& Chain : TargetChains)
		{
			if (!Chain.BoneIndices.Contains(BoneIdx))
			{
				continue;
			}
			
			if (!ChainSetupMap.Contains(&Chain))
			{
				ChainSetupMap.Emplace(&Chain, {});
			}
			ChainSetupMap[&Chain].Add(SettingsIdx);
		}
	}
	
	for (const TPair<const FResolvedBoneChain*, TArray<int32>>& ChainSetupPair : ChainSetupMap)
	{
		FPoleVectorChainIntersector& NewPVI = PoleVectorChainIntersectors.AddDefaulted_GetRef();
		NewPVI.Initialize(&Settings, ChainSetupPair.Value, ChainSetupPair.Key, InTargetSkeleton);
	}
}

FVector FIKRetargetBodyIntersectIKOp::GetTotalIntersectDelta(
	int32 CollisionProxyIndex,
	const FKShapeElem* IntersectShape,
	const FTransform& IntersectTfm,
	const TArray<FTransform>& TargetGlobalPose,
	const FTargetSkeleton& TargetSkeleton,
	FDebugBodyIntersectDrawInfo& DebugInfo)
{
	FVector IntersectDelta = FVector::ZeroVector;
	if (!Settings.bEnableGoalIntersect)
	{
		return IntersectDelta;
	}
	
	FTransform UpdateTfm = IntersectTfm;
	for (int j = 0; j < Settings.IntersectBodies.Num(); ++j)
	{
		FName BodyName = Settings.IntersectBodies[j];
		int32 BodyPoseIdx = TargetSkeleton.FindBoneIndexByName(BodyName);
		if (BodyPoseIdx == INDEX_NONE)
		{
			continue;
		}
		
		int32 BodyPhysIdx = PhysicsAsset->FindBodyIndex(BodyName);
		if (BodyPhysIdx == INDEX_NONE)
		{
			continue;
		}

		// Allow CollisionProxyIndex = -1 to skip collision-enabled check (for prop capsules)
		if (CollisionProxyIndex != INDEX_NONE && !PhysicsAsset->IsCollisionEnabled(CollisionProxyIndex, PhysicsAsset->FindBodyIndex(BodyName)))
		{
			continue;
		}
		
		FTransform BodyTfm = TargetGlobalPose[BodyPoseIdx];
		const FKShapeElem* BodyShape = FPhysShapeUtils::FindBodyShape(PhysicsAsset, BodyName);
		if (!BodyShape)
		{
			continue;
		}

		FVector DeltaDir = FVector::ZeroVector;
		double Dist = FBodyIntersectUtils::CalcIntersectionPairDelta(UpdateTfm, IntersectShape, BodyTfm, BodyShape, DeltaDir);
		if (Dist > 0.0)
		{
			IntersectDelta += Dist * DeltaDir;
			UpdateTfm.SetTranslation(IntersectTfm.GetTranslation() + IntersectDelta);
#if WITH_EDITOR
			DebugInfo.TargetIntersectDetected[j] = true;
#endif //WITH_EDITOR
		}
	}
	
	return IntersectDelta;
}

void FIKRetargetBodyIntersectIKOp::UpdatePropMoves(FName PropLinkBone, int32 IntersectIndex)
{
	if (PropLinkBone == NAME_None)
	{
		return;
	}
		
	int32 PropSetIdx = Settings.PropIntersectSettings.IndexOfByPredicate(
		[PropLinkBone](const FIKPropIntersectSettings& Elem)
		{
			return PropLinkBone == Elem.BoneName;
		});
	
	if (PropSetIdx == INDEX_NONE)
	{
		return;
	}
		
	MoveWithPropGroups[PropSetIdx].Add(IntersectIndex);
}

void FIKRetargetBodyIntersectIKOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
}

FIKRetargetOpSettingsBase* FIKRetargetBodyIntersectIKOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetBodyIntersectIKOp::GetSettingsType() const
{
	return FIKRetargetBodyIntersectIKOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetBodyIntersectIKOp::GetType() const
{
	return FIKRetargetBodyIntersectIKOp::StaticStruct();
}

const UScriptStruct* FIKRetargetBodyIntersectIKOp::GetParentOpType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

#if WITH_EDITOR

FCriticalSection FIKRetargetBodyIntersectIKOp::DebugDataMutex;

void FIKRetargetBodyIntersectIKOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	FScopeLock ScopeLock(&DebugDataMutex);

	if (!Settings.TargetPhysicsAssetOverride)
	{
		return;
	}

	for (int i = 0; i<DebugDrawInfo.TargetIntersectTfms.Num(); ++i)
	{
		const TTuple<FName, FTransform>& IntersectBody = DebugDrawInfo.TargetIntersectTfms[i];
		const FName BodyName = IntersectBody.Get<0>();
		const FKShapeElem* ShapeElem = FPhysShapeUtils::FindBodyShape(Settings.TargetPhysicsAssetOverride, BodyName);
		if (!ShapeElem)
		{
			continue;
		}

		FTransform TargetShapeTfm = IntersectBody.Get<1>();
		FVector ShapeScale = TargetShapeTfm.GetScale3D()*InComponentScale;
		TargetShapeTfm.RemoveScaling();
		FTransform CompTfm = TargetShapeTfm * InComponentTransform;
		FLinearColor TargetColor = FLinearColor::Green;
		if (DebugDrawInfo.TargetIntersectDetected[i])
		{
			TargetColor = FLinearColor::Yellow;
		}
		DebugDrawPhysBody(InPDI, CompTfm, ShapeScale, ShapeElem, TargetColor);
	}

	for (const TTuple<FName, FTransform>& GoalIntersectShape : DebugDrawInfo.GoalIntersectTfms)
	{
		const FName BodyName = GoalIntersectShape.Get<0>();
		const FKShapeElem* ShapeElem = FPhysShapeUtils::FindBodyShape(Settings.TargetPhysicsAssetOverride, BodyName);
		if (!ShapeElem)
		{
			continue;
		}
		
		FTransform GoalShapeTfm = GoalIntersectShape.Get<1>();
		FVector ShapeScale = GoalShapeTfm.GetScale3D()*InComponentScale;
		GoalShapeTfm.RemoveScaling();
		FTransform CompTfm = GoalShapeTfm * InComponentTransform;
		DebugDrawPhysBody(InPDI, CompTfm, ShapeScale, ShapeElem, FLinearColor::Red);
	}
	
	for (const TTuple<float, FTransform>& PoleIntersect : DebugDrawInfo.PoleIntersectTfms)
	{
		FKSphereElem SphereElem;
		SphereElem.Radius = PoleIntersect.Get<0>();
		
		FTransform PoleShapeTfm = PoleIntersect.Get<1>();
		FVector ShapeScale = PoleShapeTfm.GetScale3D()*InComponentScale;
		PoleShapeTfm.RemoveScaling();
		FTransform CompTfm = PoleShapeTfm * InComponentTransform;
		DebugDrawPhysBody(InPDI, CompTfm, ShapeScale, &SphereElem, FLinearColor::Red);
	}
	
	for (const TTuple<float, float, FTransform>& PropIntersect : DebugDrawInfo.PropIntersectTfms)
	{
		FKSphylElem CapsuleElem;
		CapsuleElem.Radius = PropIntersect.Get<0>();
		CapsuleElem.Length = PropIntersect.Get<1>();
		
		FTransform PropShapeTfm = PropIntersect.Get<2>();
		FVector ShapeScale = PropShapeTfm.GetScale3D()*InComponentScale;
		PropShapeTfm.RemoveScaling();
		FTransform CompTfm = PropShapeTfm * InComponentTransform;
		DebugDrawPhysBody(InPDI, CompTfm, ShapeScale, &CapsuleElem, FLinearColor(1.0, 0.25, 0.0));
	}
}

void FIKRetargetBodyIntersectIKOp::DebugDrawPhysBody(FPrimitiveDrawInterface* InPDI, const FTransform& ParentTransform, const FVector& Scale, const FKShapeElem* ShapeElem, const FLinearColor& Color) const
{
	FTransform BodyFrame = ShapeElem->GetTransform() * ParentTransform;

	const FVector Translation = BodyFrame.GetLocation();
	const FVector UnitXAxis = BodyFrame.GetUnitAxis( EAxis::X );
	const FVector UnitYAxis = BodyFrame.GetUnitAxis( EAxis::Y );
	const FVector UnitZAxis = BodyFrame.GetUnitAxis( EAxis::Z );
	
	switch (ShapeElem->GetShapeType())
	{
	case EAggCollisionShape::Box:
		{
			const FKBoxElem* BoxElem = static_cast<const FKBoxElem*>(ShapeElem);
			const FVector Extent = 0.5 * FVector(BoxElem->X, BoxElem->Y, BoxElem->Z) * Scale;
			DrawOrientedWireBox(InPDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, Extent, Color, SDPG_Foreground);
			return;
		}
	case EAggCollisionShape::Sphyl:
		{
			const FKSphylElem* CapsuleElem = static_cast<const FKSphylElem*>(ShapeElem);
			const FVector Scale3DAbs = Scale.GetAbs();
			const double Radius = CapsuleElem->GetScaledRadius(Scale3DAbs);
			const double HalfHeight = CapsuleElem->GetScaledHalfLength(Scale3DAbs);
			DrawWireCapsule(InPDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, Color,
						Radius, HalfHeight,25, SDPG_Foreground, 0, 1.0);
			return;
		}
	case EAggCollisionShape::Sphere:
		{
			const FKSphereElem* SphereElem = static_cast<const FKSphereElem*>(ShapeElem);
			const double Radius = SphereElem->Radius * Scale.GetAbsMin();
			DrawWireSphere(InPDI, Translation, Color, Radius, 25, SDPG_Foreground);
			return;
		}
	default:
		return;
	}
}

void FIKRetargetBodyIntersectIKOp::ResetDebugInfo()
{
	FScopeLock ScopeLock(&DebugDataMutex);
	
	DebugDrawInfo.TargetIntersectTfms.Reset();
	DebugDrawInfo.TargetIntersectDetected.Reset();
	DebugDrawInfo.GoalIntersectTfms.Reset();
	DebugDrawInfo.PropIntersectTfms.Reset();
}

#endif //WITH_EDITOR

FIKRetargetBodyIntersectIKOpSettings UIKRetargetBodyIntersectController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetBodyIntersectIKOpSettings*>(OpSettingsToControl);
}

void UIKRetargetBodyIntersectController::SetSettings(FIKRetargetBodyIntersectIKOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
