// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsBodyHelpers.h"

#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"

double FBodyIntersectUtils::CalcIntersectionPairDelta(const FTransform& GoalShapeTfm, const FKShapeElem* GoalShapeElem, const FTransform& TargetShapeTfm, const FKShapeElem* TargetShapeElem, FVector& OutDeltaDir)
{
	if (!GoalShapeElem || !TargetShapeElem)
	{
		return -1.0;
	}
	
	double Dist;
	EAggCollisionShape::Type GoalShapeType = GoalShapeElem->GetShapeType();
	EAggCollisionShape::Type TargetShapeType = TargetShapeElem->GetShapeType();
	bool NeedInverse;
	switch (CalcBodyIntersectPairType(GoalShapeElem, TargetShapeElem, NeedInverse))
	{
	case EBodyIntersectPairType::SphereSphere:
		{
			const FKSphereElem* SphereElem1 = static_cast<const FKSphereElem*>(GoalShapeElem);
			const FKSphereElem* SphereElem2 = static_cast<const FKSphereElem*>(TargetShapeElem);
			Dist = CalcSphereSphereShapeDelta(OutDeltaDir, GoalShapeTfm, SphereElem1, TargetShapeTfm, SphereElem2);
			break;
		}

	case EBodyIntersectPairType::SphereBox:
		{
			if (NeedInverse)
			{
				const FKSphereElem* SphereElem1 = static_cast<const FKSphereElem*>(TargetShapeElem);
				const FKBoxElem* BoxElem2 = static_cast<const FKBoxElem*>(GoalShapeElem);
				Dist = CalcSphereBoxShapeDelta(OutDeltaDir, TargetShapeTfm, SphereElem1, GoalShapeTfm, BoxElem2);
				OutDeltaDir *= -1;
			}
			else
			{
				const FKSphereElem* SphereElem1 = static_cast<const FKSphereElem*>(GoalShapeElem);
				const FKBoxElem* BoxElem2 = static_cast<const FKBoxElem*>(TargetShapeElem);
				Dist = CalcSphereBoxShapeDelta(OutDeltaDir, GoalShapeTfm, SphereElem1, TargetShapeTfm, BoxElem2);
			}
			break;
		}
		
	case EBodyIntersectPairType::SphereCapsule:
		{
			if (NeedInverse)
			{
				const FKSphereElem* SphereElem1 = static_cast<const FKSphereElem*>(TargetShapeElem);
				const FKSphylElem* CapsuleElem2 = static_cast<const FKSphylElem*>(GoalShapeElem);
				Dist = CalcSphereCapsuleShapeDelta(OutDeltaDir, TargetShapeTfm, SphereElem1, GoalShapeTfm, CapsuleElem2);
				OutDeltaDir *= -1;
			}
			else
			{
				const FKSphereElem* SphereElem1 = static_cast<const FKSphereElem*>(GoalShapeElem);
				const FKSphylElem* CapsuleElem2 = static_cast<const FKSphylElem*>(TargetShapeElem);
				Dist = CalcSphereCapsuleShapeDelta(OutDeltaDir, GoalShapeTfm, SphereElem1, TargetShapeTfm, CapsuleElem2);
			}
			break;
		}

	/*case EBodyIntersectPairType::BoxBox:
		{
			FKBoxElem* BoxElem1 = static_cast<FKBoxElem*>(GoalShapeElem);
			FKBoxElem* BoxElem2 = static_cast<FKBoxElem*>(TargetShapeElem);
			Dist = CalcBoxBoxShapeDelta(OutDeltaDir, GoalShapeTfm, BoxElem1, TargetShapeTfm, BoxElem2);
			break;
		}*/

	case EBodyIntersectPairType::BoxCapsule:
		{
			if (NeedInverse)
			{
				const FKBoxElem* BoxElem1 = static_cast<const FKBoxElem*>(TargetShapeElem);
				const FKSphylElem* CapsuleElem2 = static_cast<const FKSphylElem*>(GoalShapeElem);
				Dist = CalcBoxCapsuleShapeDelta(OutDeltaDir, TargetShapeTfm, BoxElem1, GoalShapeTfm, CapsuleElem2);
				OutDeltaDir *= -1;
			}
			else
			{
				const FKBoxElem* BoxElem1 = static_cast<const FKBoxElem*>(GoalShapeElem);
				const FKSphylElem* CapsuleElem2 = static_cast<const FKSphylElem*>(TargetShapeElem);
				Dist = CalcBoxCapsuleShapeDelta(OutDeltaDir, GoalShapeTfm, BoxElem1, TargetShapeTfm, CapsuleElem2);
			}
			break;
		}
		
	case EBodyIntersectPairType::CapsuleCapsule:
		{
			const FKSphylElem* CapsuleElem1 = static_cast<const FKSphylElem*>(GoalShapeElem);
			const FKSphylElem* CapsuleElem2 = static_cast<const FKSphylElem*>(TargetShapeElem);
			Dist = CalcCapsuleCapsuleShapeDelta(OutDeltaDir, GoalShapeTfm, CapsuleElem1, TargetShapeTfm, CapsuleElem2);
			break;
		}

	default:
		{
			ensureMsgf(false, TEXT("Unexpected Intersection Pair Type: %d - %d"), static_cast<int>(GoalShapeType), static_cast<int>(TargetShapeType));
			Dist = -1.0;
		}
	}

	return Dist;
}

double FBodyIntersectUtils::CalcIntersectionPairDelta(const FTransform& GoalShapeTfm, const FKAggregateGeom* GoalAggGeom, const FTransform& TargetShapeTfm, const FKAggregateGeom* TargetAggGeom, FVector& OutDeltaDir)
{
	if (!GoalAggGeom || !TargetAggGeom)
	{
		return -1.0;
	}
	
	double Dist = -1.0;
	FVector Delta = FVector::ZeroVector;
	FTransform UpdateTfm = GoalShapeTfm;
	int32 NumGoalShapes = GoalAggGeom->GetElementCount();
	int32 NumTargetShapes = TargetAggGeom->GetElementCount();
	for (int32 GoalShapeIdx = 0; GoalShapeIdx < NumGoalShapes; GoalShapeIdx++)
	{
		const FKShapeElem* GoalShapeElem = GoalAggGeom->GetElement(GoalShapeIdx);
		for (int32 TargetShapeIdx = 0; TargetShapeIdx < NumTargetShapes; TargetShapeIdx++)
		{
			const FKShapeElem* TargetShapeElem = TargetAggGeom->GetElement(TargetShapeIdx);
			
			FVector PairDeltaDir = FVector::ZeroVector;
			double PairDist = CalcIntersectionPairDelta(UpdateTfm, GoalShapeElem, TargetShapeTfm, TargetShapeElem, PairDeltaDir);
			
			if (PairDist > 0)
			{
				Delta += PairDist*PairDeltaDir;
				UpdateTfm.SetTranslation(GoalShapeTfm.GetTranslation() + Delta);
			}
		}
	}
	
	Dist = Delta.Size();
	OutDeltaDir = (Dist > UE_SMALL_NUMBER) ? Delta / Dist : FVector::ZeroVector;
	return Dist;
}

double FBodyIntersectUtils::SegmentDistToAABB(
	const FVector& A1, const FVector& A2,
	const FVector& BoxMin, const FVector& BoxMax,
	FVector& OutSeg, FVector& OutBox)
{
	// heuristic: try endpoints, the center projection, and orthogonal face hits
	FVector BestS = A1;
	FVector BestB(FMath::Clamp<FVector::FReal>(A1.X, BoxMin.X, BoxMax.X), FMath::Clamp<FVector::FReal>(A1.Y, BoxMin.Y, BoxMax.Y), FMath::Clamp<FVector::FReal>(A1.Z, BoxMin.Z, BoxMax.Z));
	double bestD2 = FVector::DistSquared(BestS, BestB);

	auto Consider = [&](const FVector& S, const FVector& C) {
			const double d2 = FVector::DistSquared(S, C);
			if (d2 < bestD2) { bestD2 = d2; BestS = S; BestB = C; }
	};

	// endpoints B
	Consider(A2, FVector(FMath::Clamp<FVector::FReal>(A2.X, BoxMin.X, BoxMax.X), FMath::Clamp<FVector::FReal>(A2.Y, BoxMin.Y, BoxMax.Y), FMath::Clamp<FVector::FReal>(A2.Z, BoxMin.Z, BoxMax.Z)));

	// segment point near box center
	const FVector Center = 0.5f * (BoxMin + BoxMax);
	const FVector P = FMath::ClosestPointOnSegment(Center, A1, A2);
	Consider(P, FVector(FMath::Clamp<FVector::FReal>(P.X, BoxMin.X, BoxMax.X), FMath::Clamp<FVector::FReal>(P.Y, BoxMin.Y, BoxMax.Y), FMath::Clamp<FVector::FReal>(P.Z, BoxMin.Z, BoxMax.Z)));

	// orthogonal face candidates
	const FVector A1A2 = A2 - A1;
	auto ConsiderFace = [&](int axis, double value) {
			const double dir = (axis==0? A1A2.X : axis==1? A1A2.Y : A1A2.Z);
			if (FMath::IsNearlyZero(dir))
			{
				return;
			}
			const double a0 = (axis==0? A1.X : axis==1? A1.Y : A1.Z);
			const double t  = (value - a0) / dir;
			if (t < 0.0 || t > 1.0)
			{
				return;
			}
			FVector S = A1 + t * A1A2;
			// check other two axes in range
			int u=(axis+1)%3, v=(axis+2)%3;
			auto Get=[&](const FVector& V,int k)->double{ return k==0?V.X:k==1?V.Y:V.Z; };
			if ( Get(S,u) >= Get(BoxMin,u) && Get(S,u) <= Get(BoxMax,u) &&
				 Get(S,v) >= Get(BoxMin,v) && Get(S,v) <= Get(BoxMax,v) )
			{
				FVector C = S;
				if (axis==0) C.X = value;
				else if (axis==1) C.Y = value;
				else              C.Z = value;
				Consider(S, C);
			}
	};
	ConsiderFace(0, BoxMin.X); ConsiderFace(0, BoxMax.X);
	ConsiderFace(1, BoxMin.Y); ConsiderFace(1, BoxMax.Y);
	ConsiderFace(2, BoxMin.Z); ConsiderFace(2, BoxMax.Z);

	OutSeg = BestS;
	OutBox = BestB;
	
	return FMath::Sqrt(bestD2);
}

double FBodyIntersectUtils::CalcSphereSphereShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKSphereElem* ShapeElem1, const FTransform& ShapeTfm2, const FKSphereElem* ShapeElem2)
{
	const double SphereRadius1 = ShapeTfm1.GetScale3D().GetAbsMin() * ShapeElem1->Radius;
	const FVector WorldCenter1 = ShapeTfm1.TransformPositionNoScale(ShapeElem1->Center);
	const double SphereRadius2 = ShapeTfm2.GetScale3D().GetAbsMin() * ShapeElem2->Radius;
	const FVector WorldCenter2 = ShapeTfm2.TransformPositionNoScale(ShapeElem2->Center);

	const FVector Delta = WorldCenter2 - WorldCenter1;
	const double DeltaLen = Delta.Size();
	const double DistToEdge = SphereRadius2 + SphereRadius1 - DeltaLen;

	OutDeltaDir = (DeltaLen > UE_SMALL_NUMBER) ? - Delta / DeltaLen : FVector::ZeroVector;
	return DistToEdge;
}

double FBodyIntersectUtils::CalcSphereBoxShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKSphereElem* ShapeElem1, const FTransform& ShapeTfm2, const FKBoxElem* ShapeElem2)
{
	const double SphereRadius1 = ShapeTfm1.GetScale3D().GetAbsMin() * ShapeElem1->Radius;
	const FVector WorldCenter1 = ShapeTfm1.TransformPositionNoScale(ShapeElem1->Center);
	
	FTransform ShapeTfm2NoScale = ShapeTfm2;
	ShapeTfm2NoScale.RemoveScaling();
	const FTransform LocalToWorldTM2 = ShapeElem2->GetTransform() * ShapeTfm2NoScale;
	const FVector SphereBoxWorldCenter2 = LocalToWorldTM2.InverseTransformPositionNoScale(WorldCenter1);
	const FVector HalfBoxXYZ = 0.5 * FVector(ShapeElem2->X, ShapeElem2->Y, ShapeElem2->Z) * ShapeTfm2.GetScale3D();
	
	const FVector ClosestLocalPosition(FMath::Clamp<FVector::FReal>(SphereBoxWorldCenter2.X, -HalfBoxXYZ.X, HalfBoxXYZ.X), FMath::Clamp<FVector::FReal>(SphereBoxWorldCenter2.Y, -HalfBoxXYZ.Y, HalfBoxXYZ.Y), FMath::Clamp<double>(SphereBoxWorldCenter2.Z, -HalfBoxXYZ.Z, HalfBoxXYZ.Z));
	double DistToEdge = UE_BIG_NUMBER;
	if (ClosestLocalPosition == SphereBoxWorldCenter2)
	{
		const FVector DistBoxMax = HalfBoxXYZ - SphereBoxWorldCenter2;
		const FVector DistBoxMin = HalfBoxXYZ + SphereBoxWorldCenter2;
		for (int i = 0; i < 3; i++)
		{
			if (DistToEdge > DistBoxMax[i])
			{
				DistToEdge = DistBoxMax[i];
				OutDeltaDir = FVector::ZeroVector;
				OutDeltaDir[i] = 1;
			}
			if (DistToEdge > DistBoxMin[i])
			{
				DistToEdge = DistBoxMin[i];
				OutDeltaDir = FVector::ZeroVector;
				OutDeltaDir[i] = -1;
			}
		}
		DistToEdge += SphereRadius1;
	}
	else
	{
		const FVector Delta = SphereBoxWorldCenter2-ClosestLocalPosition;
		const double DeltaLen = Delta.Size();
		DistToEdge = SphereRadius1 - DeltaLen;
		OutDeltaDir = Delta / DeltaLen;
	}
	
	OutDeltaDir = LocalToWorldTM2.TransformVectorNoScale(OutDeltaDir);
	return DistToEdge;
}

double FBodyIntersectUtils::CalcSphereCapsuleShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKSphereElem* ShapeElem1, const FTransform& ShapeTfm2, const FKSphylElem* ShapeElem2)
{
	const double SphereRadius1 = ShapeTfm1.GetScale3D().GetAbsMin() * ShapeElem1->Radius;

	FTransform ShapeTfm2NoScale = ShapeTfm2;
	ShapeTfm2NoScale.RemoveScaling();
	const FTransform LocalToWorldTM2 = ShapeElem2->GetTransform() * ShapeTfm2NoScale;
	const FVector Scale3DAbs2 = ShapeTfm2.GetScale3D().GetAbs();
	const double CapsuleRadius2 = ShapeElem2->GetScaledRadius(Scale3DAbs2);
	const double CapsuleHalfLen2 = 0.5f * ShapeElem2->GetScaledCylinderLength(Scale3DAbs2);
	
	const FVector A1 = ShapeTfm1.TransformPositionNoScale(FVector(ShapeElem1->Center));
	const FVector B1 = LocalToWorldTM2.TransformPositionNoScale(FVector(0.f, 0.f, CapsuleHalfLen2));
	const FVector B2 = LocalToWorldTM2.TransformPositionNoScale(FVector(0.f, 0.f, -CapsuleHalfLen2));
	const FVector P2 = FMath::ClosestPointOnSegment(A1, B1, B2);

	const FVector Delta = P2 - A1;
	const double DeltaLen = Delta.Size();
	const double DistToEdge = CapsuleRadius2 + SphereRadius1 - DeltaLen;
	
	OutDeltaDir = (DeltaLen > UE_SMALL_NUMBER) ? - Delta / DeltaLen : FVector::ZeroVector;
	return DistToEdge;
}

double FBodyIntersectUtils::CalcBoxBoxShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKBoxElem* ShapeElem1, const FTransform& ShapeTfm2, const FKBoxElem* ShapeElem2)
{
	// TODO: Warning for no implementation
	OutDeltaDir = FVector::ZeroVector;
	return 0;
}

double FBodyIntersectUtils::CalcBoxCapsuleShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKBoxElem* ShapeElem1, const FTransform& ShapeTfm2, const FKSphylElem* ShapeElem2)
{
	FTransform ShapeTfm1NoScale = ShapeTfm1;
	ShapeTfm1NoScale.RemoveScaling();
	const FTransform LocalToWorldTM1 = ShapeElem1->GetTransform() * ShapeTfm1NoScale;
	const FVector HalfBoxXYZ1 = 0.5 * FVector(ShapeElem1->X, ShapeElem1->Y, ShapeElem1->Z) * ShapeTfm1.GetScale3D();
	
	FTransform ShapeTfm2NoScale = ShapeTfm2;
	ShapeTfm2NoScale.RemoveScaling();
	const FTransform LocalToWorldTM2 = ShapeElem2->GetTransform() * ShapeTfm2NoScale;
	const FVector Scale3DAbs2 = ShapeTfm2.GetScale3D().GetAbs();
	const double CapsuleRadius2 = ShapeElem2->GetScaledRadius(Scale3DAbs2);
	const double CapsuleHalfLen2 = 0.5 * ShapeElem2->GetScaledCylinderLength(Scale3DAbs2);

	const FVector A1 = LocalToWorldTM1.InverseTransformPositionNoScale(LocalToWorldTM2.TransformPositionNoScale(FVector(0.f, 0.f, CapsuleHalfLen2)));
	const FVector A2 = LocalToWorldTM1.InverseTransformPositionNoScale(LocalToWorldTM2.TransformPositionNoScale(FVector(0.f, 0.f, -CapsuleHalfLen2)));

	FVector PSeg, PBox;
	double Dist = SegmentDistToAABB(A1, A2, -HalfBoxXYZ1, HalfBoxXYZ1, PSeg, PBox);
	if (Dist >= CapsuleRadius2)
	{
		return 0.;
	}
	
	if (Dist > UE_SMALL_NUMBER)
	{
		OutDeltaDir = LocalToWorldTM1.TransformVectorNoScale((PBox-PSeg)/Dist);
		return CapsuleRadius2-Dist;
	}

	// Use the segment midpoint as representative axis point
	const FVector Mid = 0.5f * (A1 + A2);
	const FVector absM(FMath::Abs(Mid.X), FMath::Abs(Mid.Y), FMath::Abs(Mid.Z));
	const FVector clear(HalfBoxXYZ1.X - absM.X, HalfBoxXYZ1.Y - absM.Y, HalfBoxXYZ1.Z - absM.Z);

	// If the axis is inside, at least one clearance should be >= 0
	// The required push along an axis = max(0, Radius - clearance)
	double PushX = CapsuleRadius2 - clear.X;
	double PushY = CapsuleRadius2 - clear.Y;
	double PushZ = CapsuleRadius2 - clear.Z;

	// pick the axis with largest deficit (most penetration), but only if positive
	int Axis = -1;
	double DistToEdge = 0;

	if (PushX > DistToEdge)
	{
		DistToEdge = PushX;
		Axis = 0;
	}
	if (PushY > DistToEdge)
	{
		DistToEdge = PushY;
		Axis = 1;
	}
	if (PushZ > DistToEdge)
	{
		DistToEdge = PushZ;
		Axis = 2;
	}
	
	FVector DeltaDir = FVector::ZeroVector;
	if (Axis >= 0 && DistToEdge > 0.f)
	{
		if (Axis == 0)
		{
			DeltaDir = FVector( (Mid.X >= 0 ? 1 : -1), 0, 0 );
		}
		else if (Axis == 1)
		{
			DeltaDir = FVector( 0, (Mid.Y >= 0 ? 1 : -1), 0 );
		}
		else
		{
			DeltaDir = FVector( 0, 0, (Mid.Z >= 0 ? 1 : -1) );
		}
	}
	
	OutDeltaDir = LocalToWorldTM1.TransformVectorNoScale(DeltaDir);
	return DistToEdge;
}

double FBodyIntersectUtils::CalcCapsuleCapsuleShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKSphylElem* ShapeElem1, const FTransform& ShapeTfm2, const FKSphylElem* ShapeElem2)
{
	FTransform ShapeTfm1NoScale = ShapeTfm1;
	ShapeTfm1NoScale.RemoveScaling();
	const FTransform LocalToWorldTM1 = ShapeElem1->GetTransform() * ShapeTfm1NoScale;
	const FVector Scale3DAbs1 = ShapeTfm1.GetScale3D().GetAbs();
	const double CapsuleRadius1 = ShapeElem1->GetScaledRadius(Scale3DAbs1);
	const double CapsuleHalfLen1 = 0.5 * ShapeElem1->GetScaledCylinderLength(Scale3DAbs1);

	FTransform ShapeTfm2NoScale = ShapeTfm2;
	ShapeTfm2NoScale.RemoveScaling();
	const FTransform LocalToWorldTM2 = ShapeElem2->GetTransform() * ShapeTfm2NoScale;
	const FVector Scale3DAbs2 = ShapeTfm2.GetScale3D().GetAbs();
	const double CapsuleRadius2 = ShapeElem2->GetScaledRadius(Scale3DAbs2);
	const double CapsuleHalfLen2 = 0.5 * ShapeElem2->GetScaledCylinderLength(Scale3DAbs2);
	
	const FVector A1 = LocalToWorldTM1.TransformPositionNoScale(FVector(0.f, 0.f, CapsuleHalfLen1));
	const FVector A2 = LocalToWorldTM1.TransformPositionNoScale(FVector(0.f, 0.f, -CapsuleHalfLen1));
	const FVector B1 = LocalToWorldTM2.TransformPositionNoScale(FVector(0.f, 0.f, CapsuleHalfLen2));
	const FVector B2 = LocalToWorldTM2.TransformPositionNoScale(FVector(0.f, 0.f, -CapsuleHalfLen2));
	FVector P1, P2;
	FMath::SegmentDistToSegmentSafe(A1, A2, B1, B2, P1, P2);

	const FVector Delta = P2 - P1;
	const double DeltaLen = Delta.Size();
	const double DistToEdge = CapsuleRadius1 + CapsuleRadius2 - DeltaLen;
	
	OutDeltaDir = (DeltaLen > UE_SMALL_NUMBER) ? - Delta / DeltaLen : FVector::ZeroVector;
	return DistToEdge;
}

EBodyIntersectPairType FBodyIntersectUtils::CalcBodyIntersectPairType(const FKShapeElem* ShapeElem1, const FKShapeElem* ShapeElem2, bool &NeedInverse)
{
	uint8 TypeIndex1 = static_cast<uint8>(ShapeElem1->GetShapeType());
	uint8 TypeIndex2 = static_cast<uint8>(ShapeElem2->GetShapeType());
	NeedInverse = TypeIndex1 > TypeIndex2;
	
	if (NeedInverse)
	{
		TypeIndex1 = static_cast<uint8>(ShapeElem2->GetShapeType());
		TypeIndex2 = static_cast<uint8>(ShapeElem1->GetShapeType());
	}
	uint8 PairTypeIndex =  TypeIndex2 * (TypeIndex2 + 1) / 2 + TypeIndex1;
	if (PairTypeIndex < static_cast<uint8>(EBodyIntersectPairType::Unknown))
	{
		return static_cast<EBodyIntersectPairType>(PairTypeIndex);
	}
	return EBodyIntersectPairType::Unknown;
}

double FPhysShapeUtils::CalcShapeSmallRadius(const FKShapeElem* ShapeElem)
{
	// Return 1.0 as default so scaling can set "size" of sphere
	const double DefaultRadius = 1.0;
	if (!ShapeElem)
	{
		return DefaultRadius;
	}
	
	switch (ShapeElem->GetShapeType())
	{
	case EAggCollisionShape::Sphere:
		{
			const FKSphereElem* SphereElem = static_cast<const FKSphereElem*>(ShapeElem);
			return SphereElem->Radius;
		}

	case EAggCollisionShape::Sphyl:
		{
			const FKSphylElem* CapsuleElem = static_cast<const FKSphylElem*>(ShapeElem);
			return CapsuleElem->Radius;
		}
	case EAggCollisionShape::Box:
		{
			const FKBoxElem* BoxElem = static_cast<const FKBoxElem*>(ShapeElem);
			return 0.5 * FMath::Min(FMath::Min(BoxElem->X, BoxElem->Y), BoxElem->Z);
		}
		// case EAggCollisionShape::Convex:
		// 	{
		// 		FKConvexElem* ConvElem = static_cast<FKConvexElem*>(ShapeElem);
		// 		return 0.5 * ConvElem->ElemBox.GetSize().GetMin();
		// 	}

	default:
		{
			return DefaultRadius;
		}
	}
}

double FPhysShapeUtils::CalcShapeAvgRadius(const FKShapeElem* ShapeElem)
{
	const double DefaultRadius = 0.0;
	if (!ShapeElem)
	{
		return DefaultRadius;
	}
	
	switch (ShapeElem->GetShapeType())
	{
	case EAggCollisionShape::Sphere:
		{
			const FKSphereElem* SphereElem = static_cast<const FKSphereElem*>(ShapeElem);
			return SphereElem->Radius;
		}

	case EAggCollisionShape::Sphyl:
		{
			const FKSphylElem* CapsuleElem = static_cast<const FKSphylElem*>(ShapeElem);
			return CapsuleElem->Radius + 0.25*CapsuleElem->Length;
		}

	default:
		{
			return 0.0;
		}
	}
}

const FKShapeElem* FPhysShapeUtils::FindBodyShape(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	int32 BodyIdx = PhysAsset->FindBodyIndex(BoneName);
	if (BodyIdx == INDEX_NONE)
	{
		UE_LOGF(LogAnimation, Warning, "No body index found: %ls", *BoneName.ToString());
		return nullptr;
	}
	return PhysAsset->SkeletalBodySetups[BodyIdx]->AggGeom.GetElement(0);
}

const FKAggregateGeom* FPhysShapeUtils::FindBodyAggGeom(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	int32 BodyIdx = PhysAsset->FindBodyIndex(BoneName);
	if (BodyIdx == INDEX_NONE)
	{
		UE_LOGF(LogAnimation, Warning, "No body index found: %ls", *BoneName.ToString());
		return nullptr;
	}
	return &PhysAsset->SkeletalBodySetups[BodyIdx]->AggGeom;
}
