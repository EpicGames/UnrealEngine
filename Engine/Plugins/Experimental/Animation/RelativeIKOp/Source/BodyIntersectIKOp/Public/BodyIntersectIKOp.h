// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/TransformVectorized.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetSettings.h"

#include "BodyIntersectIKOp.generated.h"

#define UE_API BODYINTERSECTIKOP_API

#define LOCTEXT_NAMESPACE "BodyIntersectIKOp"

struct FIKRigGoal;
struct FKShapeElem;
struct FKSphereElem;
struct FKBoxElem;
struct FKSphylElem;
struct FResolvedBoneChain;
class UPhysicsAsset;
class USkeletalMesh;

// Used to hold all debug draw info for convenience
struct FDebugBodyIntersectDrawInfo
{
	// Intersect info
	TArray<bool> TargetIntersectDetected;
	TArray<TTuple<FName,FTransform>> TargetIntersectTfms;
	TArray<TTuple<FName,FTransform>> GoalIntersectTfms;
	
	TArray<TTuple<float,float,FTransform>> PropIntersectTfms;
	
	TArray<TTuple<float,FTransform>> PoleIntersectTfms;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Prop Intersect Settings"))
struct FIKPropIntersectSettings
{
	GENERATED_BODY()
	
	// The name of the attach bone or IK bone this intersection references
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Intersect Settings")
	FName BoneName = NAME_None;
	
	// Optionally also move this bone
	// TODO: Need to figure out how to interact with blend and relative ik pin bones (RunAfterParent+offset?)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Intersect Settings")
	bool bPinBone = false;
	
	// Capsule radius for prop intersector
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Intersect Settings", meta=(ClampMin="0.0"))
	float CapsuleRadius = 0.0f;
	
	// Capsule length for prop intersector
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Intersect Settings", meta=(ClampMin="0.0"))
	float CapsuleLength = 0.0f;
	
	// Capsule orientation relative to bone
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Intersect Settings")
	FRotator CapsuleRotation = FRotator::ZeroRotator;
	
	// Capsule translation relative to bone
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Intersect Settings")
	FVector CapsuleTranslation = FVector::ZeroVector;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Goal Intersect Settings"))
struct FIKGoalIntersectShapeSettings
{
	GENERATED_BODY()

	// The IK Goal Name (See Run IK Rig Op) to intersect
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Goal Intersect Settings")
	FName Goal = NAME_None;
	// Shape scale , Todo: need to be updated with shape property
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Goal Intersect Settings", meta = (ClampMin="0.0"))
	FVector GoalShapeScale = FVector::OneVector;
	// Local offset of shape from goal bone
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Goal Intersect Settings")
	FVector GoalShapeOffset = FVector::ZeroVector;
	// Prop bone/IK bone to move together with (must be in prop intersect list!)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Goal Intersect Settings")
	FName PropIntersectBone = NAME_None;
	// Alpha blend on bone motion relative to prop bone
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Goal Intersect Settings", meta=(ClampMin="0.0", ClampMax="1.0"))
	double BlendMultiMaxOffset = 0.0;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Pole Vector Settings"))
struct FIKPoleVectorIntersectSettings
{
	GENERATED_BODY()

	// The body along ik chain to check intersections and apply pole vector rotation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pole Vector Intersect Settings", meta=(ReinitializeOnEdit))
	FName BodyName = NAME_None;
	
	// Sphere size scale (base radius is body min radius or 1.0) for intersection checks
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pole Vector Intersect Settings", meta = (ClampMin="0.0"))
	double SphereScale = 1.0;
	
	// Local sphere offset from body bone
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pole Vector Intersect Settings")
	FVector SphereOffset = FVector::ZeroVector;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Body Intersect IK Settings"))
struct FIKRetargetBodyIntersectIKOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	// Target physics asset for checking intersections against
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta=(ReinitializeOnEdit))
	TObjectPtr<UPhysicsAsset> TargetPhysicsAssetOverride;

	// Apply intersection delta to goals 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(ReinitializeOnEdit))
	bool bEnableGoalIntersect = true;
	
	// IK Goals Shape intersect against physics asset bodies 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(ReinitializeOnEdit))
	TArray<FIKGoalIntersectShapeSettings> IntersectGoalSettings;
	
	// Prop intersection shape and bone info (must match goal intersect names if moving together) 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(ReinitializeOnEdit))
	TArray<FIKPropIntersectSettings> PropIntersectSettings;

	// Bone names with attached physics bodies to do trivial intersection against
	// **NOTE: Only Sphere, Capsule and Box bodies currently supported**
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(ReinitializeOnEdit, NotOverrideable))
	TArray<FName> IntersectBodies;
	
	// Check bodies in Pole Vector Intersect Settings for intersection and try to fixup using pole vector rotation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pole Vector", meta=(ReinitializeOnEdit))
	bool bEnablePoleVectorIntersect = true;
	
	// Body settings for pole vector intersection fixup
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pole Vector", meta=(ReinitializeOnEdit))
	TArray<FIKPoleVectorIntersectSettings> PoleVectorIntersectSettings;
};

// Per-chain pole vector intersection functionality
struct FPoleVectorChainIntersector
{
	void Initialize(
		const FIKRetargetBodyIntersectIKOpSettings* InSettings,
		const TArray<int32>& InBodySettingsIdx,
		const FResolvedBoneChain* InTargetChain,
		const FTargetSkeleton& InTargetSkeleton);
	
	void ApplyIntersectRotation(
		const FRetargetSkeleton& TargetSkeleton,
		TArray<FTransform>& OutTargetGlobalPose,
		FDebugBodyIntersectDrawInfo& DebugInfo) const;
	
	FVector GetIntersectDelta(
		const FTransform& PoleTfm,
		FKSphereElem* PoleSphere,
		const TArray<FTransform>& TargetGlobalPose,
		FDebugBodyIntersectDrawInfo& DebugInfo) const;
	
	FVector CalculatePoleVector(
		const EAxis::Type& PoleAxis,
		const TArray<int32>& BoneIndices,
		const TArray<FTransform>& GlobalPose) const;
	
	FVector GetChainAxisNormalized(
		const TArray<int32>& BoneIndices,
		const TArray<FTransform>& GlobalPose) const;
	
	TObjectPtr<const UPhysicsAsset> PhysicsAsset;
	
	const FIKRetargetBodyIntersectIKOpSettings* Settings;
	const FResolvedBoneChain* TargetBoneChain;
	
	TArray<int32> PoleBodiesSettingsIdx;
	TArray<int32> PoleBonesIdx;
	TArray<int32> IntersectBonesIdx;
	
	TArray<int32> AllChildrenWithinChain;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Body Intersect Goals"))
struct FIKRetargetBodyIntersectIKOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;
	
	UE_API virtual void RunAfterParent(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;
	
	UE_API virtual const UScriptStruct* GetParentOpType() const override;

	UPROPERTY()
	FIKRetargetBodyIntersectIKOpSettings Settings;

private:
	struct FIntersectFrameDeltaInfo
	{
		FIKRigGoal* GoalPtr = nullptr;
		FTransform StartBodyTfm = FTransform::Identity;
		FVector StartGoalLoc = FVector::ZeroVector;
		FVector IntersectDelta = FVector::ZeroVector;
	};
	
	void InitPoleVectorIntersectors(const TArray<FResolvedBoneChain>& TargetChains, const FTargetSkeleton& InTargetSkeleton);
	
	FVector GetTotalIntersectDelta(
		int32 CollisionProxyIndex,
		const FKShapeElem* IntersectShape,
		const FTransform& IntersectTfm,
		const TArray<FTransform>& TargetGlobalPose,
		const FTargetSkeleton& TargetSkeleton,
		FDebugBodyIntersectDrawInfo& DebugInfo);
	
	void UpdatePropMoves(FName PropLinkBone, int32 IntersectIndex);

	FVector GoalLocBlendCompSpace(const FIKRigGoal* Goal, const FTransform& BoneTfm) const;
	void SetGoalPosFromCompSpace(FIKRigGoal* Goal, const FTransform& BoneTfm, const FVector& CompSpaceLoc) const;

#if WITH_EDITOR
public:
	UE_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;

	virtual bool HasDebugDrawing() override { return true; };

private:
	void DebugDrawPhysBody(FPrimitiveDrawInterface* InPDI, const FTransform& ParentTransform, const FVector& Scale, const FKShapeElem* ShapeElem, const FLinearColor& Color) const;
	
	void ResetDebugInfo();
	
	FKShapeElem* CopyShape(FKShapeElem* InShape);

	// Collection of all debug structures
	FDebugBodyIntersectDrawInfo DebugDrawInfo;
	
	static UE_API FCriticalSection DebugDataMutex;
#endif
private:
	TObjectPtr<const UPhysicsAsset> PhysicsAsset;

	// Cache ik goals to check
	TArray<const FIKRigGoal*> IntersectIKGoals;
	
	// Array (per-chain w/ pv bodies) of intersectors
	TArray<FPoleVectorChainIntersector> PoleVectorChainIntersectors;
	
	// Attach bone updates to pin
	// TODO: Decide if these should be deltas or absolute locs
	TArray<TPair<int32,FVector>> UpdatePropDeltas;
	
	// Per-frame goal deltas
	TArray<FIntersectFrameDeltaInfo> FrameDeltaInfo;
	
	// Move with prop groups
	TArray<TArray<int32>> MoveWithPropGroups;
};

/* The blueprint/python API for editing BodyIntersectIK Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetBodyIntersectController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current op settings as a struct.
	 * @return FIKRetargetBodyIntersectIKOp struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetBodyIntersectIKOpSettings GetSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRetargetBodyIntersectIKOp struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetBodyIntersectIKOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
