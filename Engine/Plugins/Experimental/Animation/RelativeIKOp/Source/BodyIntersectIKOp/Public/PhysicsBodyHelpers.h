// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Math/Transform.h"

#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"


struct FKAggregateGeom;
struct FKShapeElem;
struct FKSphereElem;
struct FKBoxElem;
struct FKSphylElem;
class UPhysicsAsset;

enum class EBodyIntersectPairType : uint8 
{
	SphereSphere,
	SphereBox,
	BoxBox,
	SphereCapsule,
	BoxCapsule,
	CapsuleCapsule,
	
	Unknown
};

/**
 * Helper static class providing physics body intersection delta functions
 */
class BODYINTERSECTIKOP_API FBodyIntersectUtils
{
public:
	static double CalcIntersectionPairDelta(const FTransform& GoalShapeTfm, const FKShapeElem* GoalShapeElem, const FTransform& TargetShapeTfm, const FKShapeElem* TargetShapeElem, FVector& OutDeltaDir);
	static double CalcIntersectionPairDelta(const FTransform& GoalShapeTfm, const FKAggregateGeom* GoalAggGeom, const FTransform& TargetShapeTfm, const FKAggregateGeom* TargetAggGeom, FVector& OutDeltaDir);
private:
	static EBodyIntersectPairType CalcBodyIntersectPairType(const FKShapeElem* ShapeElem1, const FKShapeElem* ShapeElem2, bool &NeedInverse);
	
	static double CalcSphereSphereShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKSphereElem* ShapeElem1, const FTransform& ShapeTfm2, const FKSphereElem* ShapeElem2);
	static double CalcSphereBoxShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKSphereElem* ShapeElem1, const FTransform& ShapeTfm2, const FKBoxElem* ShapeElem2);
	static double CalcSphereCapsuleShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKSphereElem* ShapeElem1, const FTransform& ShapeTfm2, const FKSphylElem* ShapeElem2);
	static double CalcBoxBoxShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKBoxElem* ShapeElem1, const FTransform& ShapeTfm2, const FKBoxElem* ShapeElem2);
	static double CalcBoxCapsuleShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKBoxElem* ShapeElem1, const FTransform& ShapeTfm2, const FKSphylElem* ShapeElem2);
	static double CalcCapsuleCapsuleShapeDelta(FVector& OutDeltaDir, const FTransform& ShapeTfm1, const FKSphylElem* ShapeElem1, const FTransform& ShapeTfm2, const FKSphylElem* ShapeElem2);

	static double SegmentDistToAABB(const FVector& A, const FVector& B, const FVector& BoxMin, const FVector& BoxMax, FVector& OutSeg, FVector& OutBox);
};

class BODYINTERSECTIKOP_API FPhysShapeUtils
{
public:
	static double CalcShapeSmallRadius(const FKShapeElem* ShapeElem);
	static double CalcShapeAvgRadius(const FKShapeElem* ShapeElem);

	static const FKShapeElem* FindBodyShape(const UPhysicsAsset* PhysAsset, FName BoneName);
	static const FKAggregateGeom* FindBodyAggGeom(const UPhysicsAsset* PhysAsset, FName BoneName);
};
