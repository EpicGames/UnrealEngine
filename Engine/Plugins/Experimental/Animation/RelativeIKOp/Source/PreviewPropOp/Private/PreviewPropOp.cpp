// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreviewPropOp.h"

#include "Retargeter/IKRetargetProcessor.h"

#if WITH_EDITOR
#include "AnimationRuntime.h"
#include "BoneContainer.h"
#include "BoneIndices.h"
#include "BonePose.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "SkeletalMeshAttributes.h"
#include "StaticMeshAttributes.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceBase.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/EngineTypes.h"
#include "Rendering/SkeletalMeshRenderData.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PreviewPropOp)

#define LOCTEXT_NAMESPACE "PreviewPropOp"

const UClass* FIKRetargetPreviewPropOpSettings::GetControllerType() const
{
	return UIKRetargetPreviewPropController::StaticClass();
}

void FIKRetargetPreviewPropOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	static TArray<FName> PropertiesToIgnore = {"AttachMapping", "PreviewProps"};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetPreviewPropOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

#if WITH_EDITOR
USkeleton* FIKRetargetPreviewPropOpSettings::GetSkeleton(const FName InPropertyName)
{
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FPreviewPropsData, SourceAttachBone))
	{
		return const_cast<USkeleton*>(SourceSkeletonAsset);
	}

	return nullptr;
}
#endif //WITH_EDITOR

bool FIKRetargetPreviewPropOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;
#if WITH_EDITOR
	PropMeshData.Reset();
	
	if (Settings.PreviewProps.IsEmpty())
	{
		return false;
	}
	
	TObjectPtr<UMaterialInterface> PropDefMaterialInterface = Settings.DebugMaterial;
	if (!PropDefMaterialInterface)
	{
		PropDefMaterialInterface = LoadObject<UMaterialInterface>(nullptr, TEXT("/Engine/EditorMaterials/Cloth/CameraLitDoubleSided.CameraLitDoubleSided"), nullptr, LOAD_None, nullptr);
	}

	if (!PropDefMaterialInterface)
	{
		return false;
	}

	PropDefaultMaterial = TStrongObjectPtr(UMaterialInstanceDynamic::Create(PropDefMaterialInterface, GetTransientPackage()));
	if (!PropDefaultMaterial)
	{
		return false;
	}
	
	for (int32 PropIdx = 0; PropIdx < Settings.PreviewProps.Num(); PropIdx++)
	{
		FPreviewPropsData& PreviewData = Settings.PreviewProps[PropIdx];
		if (!PreviewData.ShowProp)
		{
			continue;
		}
		
		if ((!PreviewData.PropStaticMeshAsset && !PreviewData.PropSkeletalMeshAsset) || PreviewData.SourceAttachBone.BoneName == NAME_None)
		{
			continue;
		}
		
		int32 SourceAttachIdx = InSourceSkeleton.FindBoneIndexByName(PreviewData.SourceAttachBone.BoneName);
		if (SourceAttachIdx == INDEX_NONE)
		{
			continue;
		}

		FName TargetAttachBone = ApplyBoneMap(PreviewData.SourceAttachBone.BoneName);
		int32 TargetAttachIdx = InTargetSkeleton.FindBoneIndexByName(TargetAttachBone);

		FPropMeshData NewMesh;
		NewMesh.PropIdx = PropIdx;
		NewMesh.SourceAttachBoneIndex = SourceAttachIdx;
		NewMesh.TargetAttachBoneIndex = TargetAttachIdx;
		NewMesh.AttachTransform = PreviewData.AttachTransform;
		
		if (GetMeshVertTris(NewMesh, PreviewData))
		{
			FScopeLock ScopeLock(&DebugDataMutex);
			PropMeshData.Add(NewMesh);
		}
	}

	bIsInitialized = !PropMeshData.IsEmpty();
#endif //WITH_EDITOR
	return bIsInitialized;
}

void FIKRetargetPreviewPropOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
#if WITH_EDITOR
	if (!bIsInitialized)
	{
		return;
	}

	double SourceScale = InProcessor.GetSourceScaleFactor();
	FTransform SourceScaleTfm{FQuat::Identity, FVector::ZeroVector, FVector(SourceScale)};
	
	FScopeLock ScopeLock(&DebugDataMutex);
	
	DebugPropInfo.PropTfmInfo.Reset(PropMeshData.Num());
	
	for (FPropMeshData& PropMesh : PropMeshData)
	{
		FPropTfmInfo& PropInfo = DebugPropInfo.PropTfmInfo.AddDefaulted_GetRef();
		
		FTransform SourceBoneTfm = InSourceGlobalPose[PropMesh.SourceAttachBoneIndex];
		PropInfo.SourceTfm = PropMesh.AttachTransform * SourceScaleTfm * SourceBoneTfm;

		PropInfo.bValidTargetTfm = (PropMesh.TargetAttachBoneIndex != INDEX_NONE);
		if (PropInfo.bValidTargetTfm)
		{
			FTransform TargetBoneTfm = OutTargetGlobalPose[PropMesh.TargetAttachBoneIndex];
			PropInfo.TargetTfm = PropMesh.AttachTransform * TargetBoneTfm;
		}
		
		// TODO: Move this into processing for props w/ anim seq only 
		UAnimSequence* PropAnimation = Settings.PreviewProps[PropMesh.PropIdx].PropAnimSequence;
		USkeletalMesh* PropSkeletalMeshAsset = Settings.PreviewProps[PropMesh.PropIdx].PropSkeletalMeshAsset;
		if (PropAnimation && PropSkeletalMeshAsset)
		{
			float TestPropAnimLength = PropAnimation->GetPlayLength();
			float TestPropPlayhead = ComputePropAnimPlayhead(TestPropAnimLength, Settings.PreviewProps[PropMesh.PropIdx].PropAnimStartTime, Settings.PreviewProps[PropMesh.PropIdx].PropAnimIsLooping);
			if (TestPropPlayhead<0.f)
			{
				continue;
			}
			
			TArray<FMatrix44f> CacheToLocals;
			TArray<FVector3f> PropVLocations;
			GetRefToAnimPoseMatrices(PropAnimation, TestPropPlayhead, PropSkeletalMeshAsset, CacheToLocals);
			UpdateSkinnedVertices(PropSkeletalMeshAsset, PropMesh, CacheToLocals);
		}
	}
	
#endif // WITH_EDITOR
}

void FIKRetargetPreviewPropOp::AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent)
{
#if WITH_EDITOR
	if (!bIsInitialized)
	{
		return;
	}

	UAnimInstance* SourceAnimInstance = SourceMeshComponent.GetAnimInstance();
	if (!SourceAnimInstance)
	{
		return;
	}
	
	// Use full montage time
	const FAnimMontageInstance* MontageInstance = SourceAnimInstance->GetActiveMontageInstance();
	if (MontageInstance)
	{
		// TODO: DeltaTimeRecord doesn't seem to work for montages when scrubbing
		CacheSourceCharacterAnim = MontageInstance->Montage;
		SourceAnimPlayhead = MontageInstance->GetPosition();
		return;
	}
	
	UpdateAnimSeqPlayhead(SourceAnimInstance);
#endif //WITH_EDITOR
}

FIKRetargetOpSettingsBase* FIKRetargetPreviewPropOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetPreviewPropOp::GetSettingsType() const
{
	return FIKRetargetPreviewPropOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetPreviewPropOp::GetType() const
{
	return FIKRetargetPreviewPropOp::StaticStruct();
}

FName FIKRetargetPreviewPropOp::ApplyBoneMap(FName SourceBone)
{
	if (Settings.AttachMapping.IsEmpty() || !Settings.AttachMapping.Contains(SourceBone))
	{
		return SourceBone;
	}

	return Settings.AttachMapping[SourceBone];
}

#if WITH_EDITOR
// -----------------
// All functionality is editor-only since at runtime op does nothing

void FIKRetargetPreviewPropOp::UpdateAnimSeqPlayhead(UAnimInstance* SourceAnimInstance)
{
	// Get source tick records
	TMap<FName, FAnimGroupInstance> SyncGroupTickRecords = SourceAnimInstance->GetSyncGroupMapRead();
	TArray<FAnimTickRecord> UngroupedTickRecords = SourceAnimInstance->GetUngroupedActivePlayersRead();
	
	auto FindFirstTickRecord = [](const TArray<FAnimTickRecord>& ActivePlayers)
		{
			for (const FAnimTickRecord& ActivePlayer : ActivePlayers)
			{
				if (Cast<UAnimSequence>(ActivePlayer.SourceAsset) != nullptr)
				{
					// Used to determine which notifies should we considered "active".
					check(ActivePlayer.DeltaTimeRecord->IsPreviousValid())
					return &ActivePlayer;
				}
			}
			
			return static_cast<const FAnimTickRecord*>(nullptr);
		};
	
	const FAnimTickRecord* FoundTickRecord = nullptr;
	// Find all the anim sequences (with sync groups) we updated this tick.
	for (const TTuple<FName, FAnimGroupInstance>& SyncGroupPair : SyncGroupTickRecords)
	{
		FoundTickRecord = FindFirstTickRecord(SyncGroupPair.Value.ActivePlayers);
		if (FoundTickRecord)
		{
			break;
		}
	}
	
	if (!FoundTickRecord)
	{
		// Find all the anim sequences (no sync groups) we updated this tick.
		FoundTickRecord = FindFirstTickRecord(UngroupedTickRecords);
	}
	
	if (!FoundTickRecord)
	{
		CacheSourceCharacterAnim = nullptr;
		SourceAnimPlayhead = 0.0f; 
	}
	else
	{
		CacheSourceCharacterAnim = Cast<UAnimSequenceBase>(FoundTickRecord->SourceAsset); 
		SourceAnimPlayhead = FoundTickRecord->DeltaTimeRecord->GetPrevious() + FoundTickRecord->DeltaTimeRecord->Delta;
	}
}

float FIKRetargetPreviewPropOp::ComputePropAnimPlayhead(float PropAnimLength, float PropAnimStartTime, bool bLooping) const
{
	if (!CacheSourceCharacterAnim)
	{
		return 0.0f;
	}
	
	if (!bLooping)
	{
		return FMath::Clamp(SourceAnimPlayhead-PropAnimStartTime, 0.0f, PropAnimLength);
	}
	
	// NOTE: This will pop at the end of char sequence if SourceAnimLength is not an integer multiple of PropAnimLength
	// TODO: Potentially compute a playrate to force correct looping behaviour? 
	return FMath::Max(FMath::Fmod(SourceAnimPlayhead-PropAnimStartTime, PropAnimLength), 0.0f);
}

void FIKRetargetPreviewPropOp::GetAnimSeqFramePose(const UAnimSequenceBase* AnimSeq, double Time, TArray<FName>& OutBones, TArray<FTransform>& OutPose) const
{
	const FReferenceSkeleton& Skel = AnimSeq->GetSkeleton()->GetReferenceSkeleton();
	int32 NumBones = Skel.GetNum();
	
	OutBones.Reset(NumBones);
	OutPose.Reset(NumBones);

	// Init bone container for pulling animseq data
	TArray<FBoneIndexType> BoneIndices;
	BoneIndices.Reserve(NumBones);
	for (FBoneIndexType BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		BoneIndices.Add(BoneIndex);
		OutBones.Add(Skel.GetBoneName(BoneIndex));
	}
	
	FBoneContainer BoneContainer;
	BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *AnimSeq->GetSkeleton());
	BoneContainer.SetUseRAWData(false);
	
	// Setup bone container
	FCompactPose CompactPose;
	FBlendedCurve Curves;
	UE::Anim::FStackAttributeContainer Attributes;
	FAnimationPoseData FramePoseData(CompactPose, Curves, Attributes);
	
	CompactPose.SetBoneContainer(&BoneContainer);
	Curves.InitFrom(BoneContainer);
	
	FAnimExtractContext FrameExtractionCtx(Time, false);
	AnimSeq->GetAnimationPose(FramePoseData, FrameExtractionCtx);
	
	FramePoseData.GetPose().CopyBonesTo(OutPose);
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const int32 ParentBoneIndex = Skel.GetParentIndex(BoneIndex);

		FTransform ParentTransform = FTransform::Identity;
		if (ParentBoneIndex != INDEX_NONE)
		{
			// TODO: Double-check parents always befor children in list
			FString ParentBoneName = Skel.GetBoneName(ParentBoneIndex).ToString();
			ParentTransform = OutPose[ParentBoneIndex];
		}
		
		OutPose[BoneIndex] = OutPose[BoneIndex] * ParentTransform;
	}
}

bool FIKRetargetPreviewPropOp::GetMeshVertTris(FPropMeshData& OutMeshData, const FPreviewPropsData& PreviewProp)
{
	const FMeshDescription* Desc = PreviewProp.GetMeshDescription(0);
	if (!Desc)
	{
		return false;
	}

	int32 NumVerts = Desc->Vertices().Num();
	int32 NumTris = Desc->Triangles().Num();
	
	OutMeshData.Vertices.Reset(NumVerts);
	OutMeshData.Vertices.AddUninitialized(NumVerts);
	OutMeshData.RestVLocation.Reset(NumVerts);
	OutMeshData.RestVLocation.AddUninitialized(NumVerts);
	OutMeshData.Indices.SetNumUninitialized(3*NumTris);

	// TODO: Almost certainly incorrectly "welding" all vert instances into whatever the first seen vert is!
	TVertexInstanceAttributesConstRef<FVector2f> TexCoords = Desc->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	TVertexInstanceAttributesConstRef<FVector3f> Tangents = Desc->VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);
	TVertexInstanceAttributesConstRef<FVector3f> Normals = Desc->VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
	TVertexInstanceAttributesConstRef<float> BinormalSigns = Desc->VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign);

	int32 TriIdx = 0;
	TMap<FVertexID, int32> TriVertMap;
	for (const FTriangleID TriID : Desc->Triangles().GetElementIDs())
	{
		TConstArrayView<FVertexInstanceID> TriInst = Desc->GetTriangleVertexInstances(TriID);
		TConstArrayView<FVertexID> TriVerts = Desc->GetTriangleVertices(TriID);

		for (int32 Idx=0; Idx < 3; ++Idx)
		{
			const FVertexID VertID = TriVerts[Idx];
			if (!TriVertMap.Contains(VertID))
			{
				FDynamicMeshVertex NewVert{Desc->GetVertexPosition(VertID)};
				if (TexCoords.IsValid())
				{
					// Max channels supported is 8
					int32 TexChannels = FMath::Min(TexCoords.GetNumChannels(), 8);
					for (int32 ChanIdx = 0; ChanIdx < TexChannels; ++ChanIdx)
					{
						NewVert.TextureCoordinate[ChanIdx] = TexCoords.Get(TriInst[Idx],ChanIdx);
					}
				}
				if (Tangents.IsValid() && Normals.IsValid() && BinormalSigns.IsValid())
				{
					FVector3f Vtan = Tangents.Get(TriInst[Idx],0);
					FVector3f Vnormal = Normals.Get(TriInst[Idx],0);
					float Vsgn = BinormalSigns.Get(TriInst[Idx],0);
					NewVert.TangentX = Vtan;
					NewVert.TangentZ = Vnormal;
					NewVert.TangentZ.Vector.W = static_cast<int8>(Vsgn);
				}

				OutMeshData.Vertices[VertID] = NewVert;
				OutMeshData.RestVLocation[VertID] = Desc->GetVertexPosition(VertID);
			}
			OutMeshData.Indices[TriIdx*3 + Idx] = VertID;
		}
		
		++TriIdx;
	}

	return true;
}



FCriticalSection FIKRetargetPreviewPropOp::DebugDataMutex;

void FIKRetargetPreviewPropOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	FScopeLock ScopeLock(&DebugDataMutex);
	
	if (!Settings.bDebugDraw)
	{
		return;
	}

	if (!bIsInitialized)
	{
		return;
	}

	// Breakpoints or slow updates can have a debug call run between init/run calls
	if (DebugPropInfo.PropTfmInfo.Num() != PropMeshData.Num())
	{
		return;
	}

	// TODO: Seems ridiculous to keep a strong and weak ptr, but this seems to stop crash on exit (look into GC of material)
	if (!PropDefaultMaterial.IsValid())
	{
		return;
	}

	FTransform SourceNoScaleTfm = FTransform(InSourceTransform.GetRotation(), InSourceTransform.GetTranslation());

	int PropIdx = 0;
	for (const FPropTfmInfo& PropInfo : DebugPropInfo.PropTfmInfo)
	{
		const FPropMeshData& PropMesh = PropMeshData[PropIdx];
		if (Settings.bShowSourceProps)
		{
			FDynamicMeshBuilder MeshBuilder(InPDI->View->GetFeatureLevel());
			{
				MeshBuilder.AddVertices(PropMesh.Vertices);
				MeshBuilder.AddTriangles(PropMesh.Indices);
			}
			
			FTransform SourceWorldTfm = DebugPropInfo.PropTfmInfo[PropIdx].SourceTfm * SourceNoScaleTfm;
			
			const FMaterialRenderProxy* MaterialProxy = PropDefaultMaterial->GetRenderProxy();
			if (MaterialProxy)
			{
				MeshBuilder.Draw(InPDI, SourceWorldTfm.ToMatrixWithScale(), MaterialProxy, SDPG_World, false);
			}
		}

		if (PropInfo.bValidTargetTfm)
		{
			FDynamicMeshBuilder MeshBuilder(InPDI->View->GetFeatureLevel());
			{
				MeshBuilder.AddVertices(PropMesh.Vertices);
				MeshBuilder.AddTriangles(PropMesh.Indices);
			}

			FTransform TargetWorldTfm = DebugPropInfo.PropTfmInfo[PropIdx].TargetTfm * InComponentTransform;
			
			const FMaterialRenderProxy* MaterialProxy = PropDefaultMaterial->GetRenderProxy();
			if (MaterialProxy)
			{
				MeshBuilder.Draw(InPDI, TargetWorldTfm.ToMatrixWithScale(), MaterialProxy, SDPG_World, false);
			}
		}
		
		++PropIdx;
	}
}

void FIKRetargetPreviewPropOp::UpdateSkinnedVertices(USkeletalMesh* SkeletalMeshAsset, FPropMeshData& PropMeshDataIn, const TArray<FMatrix44f>& RefToPoseMatrices)
{
	FMeshDescription* MeshDescriptionPtr = SkeletalMeshAsset->GetMeshDescription(0);
	if (!MeshDescriptionPtr)
	{
		return;
	}
	const int32 NumVertices = MeshDescriptionPtr->Vertices().Num();

	//Extract skin weights
	FReferenceSkeleton& RefSkeleton = SkeletalMeshAsset->GetRefSkeleton();
	FSkeletalMeshAttributes MeshAttribs(*MeshDescriptionPtr);
	FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();
	
	//CPU skinned vertices
	const int32 NumBones = MeshAttribs.GetNumBones();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		FVector3f NewPosition = FVector3f::ZeroVector;
		FVertexBoneWeights BoneWeights = VertexSkinWeights.Get(FVertexID(VertexIndex));
		const int32 InfluenceCount = BoneWeights.Num();
		for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
		{
			int32 InfluenceBoneIndex = BoneWeights[InfluenceIndex].GetBoneIndex();
			float InfluenceBoneWeight = BoneWeights[InfluenceIndex].GetWeight();

			const FMatrix44f& RefToPose = RefToPoseMatrices[InfluenceBoneIndex];
			NewPosition += RefToPose.TransformPosition(PropMeshDataIn.RestVLocation[VertexIndex]) * InfluenceBoneWeight;
		}
		PropMeshDataIn.Vertices[VertexIndex].Position = NewPosition;
	}
}

void FIKRetargetPreviewPropOp::GetRefToAnimPoseMatrices(UAnimSequence* InAnimation, float AnimPlayhead, USkeletalMesh* SkeletalMeshAsset, TArray<FMatrix44f>& OutRefToPose) const
{
	const FReferenceSkeleton& RefSkeleton = SkeletalMeshAsset->GetRefSkeleton();
	const TArray<FMatrix44f>& RefBasesInvMatrix = SkeletalMeshAsset->GetRefBasesInvMatrix();
	
	OutRefToPose.Init(FMatrix44f::Identity, RefBasesInvMatrix.Num());
	
	TArray<FName> AnimBoneNames;
	TArray<FTransform> AnimPose;
	GetAnimSeqFramePose(InAnimation, AnimPlayhead, AnimBoneNames, AnimPose);
	
	const FSkeletalMeshLODRenderData& LOD = SkeletalMeshAsset->GetResourceForRendering()->LODRenderData[0];
	const TArray<FBoneIndexType>& RequiredBoneIndices = LOD.ActiveBoneIndices;
	for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndices.Num(); BoneIndex++)
	{
		const int32 ThisBoneIndex = RequiredBoneIndices[BoneIndex];
		if (RefBasesInvMatrix.IsValidIndex(ThisBoneIndex))
		{
			FName ThisBoneName = RefSkeleton.GetBoneName(ThisBoneIndex);
			int32 PoseIdx = AnimBoneNames.Find(ThisBoneName);
			if (PoseIdx == INDEX_NONE)
			{
				continue;
			}
			
			OutRefToPose[ThisBoneIndex] = static_cast<FMatrix44f>(AnimPose[PoseIdx].ToMatrixWithScale());
		}
	}

	for (int32 ThisBoneIndex = 0; ThisBoneIndex < OutRefToPose.Num(); ++ThisBoneIndex)
	{
		OutRefToPose[ThisBoneIndex] = RefBasesInvMatrix[ThisBoneIndex] * OutRefToPose[ThisBoneIndex];
	}
}
#endif //WITH_EDITOR

FIKRetargetPreviewPropOpSettings UIKRetargetPreviewPropController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetPreviewPropOpSettings*>(OpSettingsToControl);
}

void UIKRetargetPreviewPropController::SetSettings(FIKRetargetPreviewPropOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
