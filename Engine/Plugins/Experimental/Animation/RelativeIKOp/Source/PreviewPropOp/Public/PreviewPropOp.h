// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DynamicMeshBuilder.h"
#include "Animation/BoneReference.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Retargeter/IKRetargetOps.h"

#include "PreviewPropOp.generated.h"

#define UE_API PREVIEWPROPOP_API

#define LOCTEXT_NAMESPACE "PreviewPropOp"

struct FIKRigGoal;
struct FKShapeElem;
class UAnimInstance;
class UAnimSequence;
class UAnimSequenceBase;
class UPhysicsAsset;
class USkeletalMeshComponent;

struct FPropTfmInfo
{
	bool bValidTargetTfm = false;
	FTransform SourceTfm;
	FTransform TargetTfm;
};

// Used to hold all debug draw info for convenience
struct FDebugPropDrawInfo
{
	TArray<FPropTfmInfo> PropTfmInfo;
};

struct FPropMeshData
{
	int32 PropIdx;
	TArray<uint32> Indices;
	TArray<FDynamicMeshVertex> Vertices;
	TArray<FVector3f> RestVLocation;

	int32 SourceAttachBoneIndex;
	int32 TargetAttachBoneIndex;
	FTransform AttachTransform = FTransform::Identity;
};

USTRUCT(BlueprintType)
struct FPreviewPropsData
{
	GENERATED_BODY()

// #if WITH_EDITORONLY_DATA

	// Prop mesh to anim sequence is looping
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	bool ShowProp = true;
	
	// Prop mesh to preview
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	TObjectPtr<UStaticMesh> PropStaticMeshAsset;
	
	// Prop mesh to preview
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	TObjectPtr<USkeletalMesh> PropSkeletalMeshAsset;

	// Source skeletal mesh bone to attach prop to
	UPROPERTY(EditAnywhere, Category = "Prop", meta=(ReinitializeOnEdit))
	FBoneReference SourceAttachBone;

	// Local transform of prop relative to attach bone
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	FTransform AttachTransform;
	
	// Prop mesh to anim sequence
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	TObjectPtr<UAnimSequence> PropAnimSequence;
	
	// Prop mesh to anim sequence start time
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	float PropAnimStartTime = 0.0f;
	
	// Prop mesh to anim sequence is looping
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	bool PropAnimIsLooping = true;
// ##endif //WITH_EDITORONLY_DATA

#if WITH_EDITOR
	const FMeshDescription* GetMeshDescription(int32 LodIndex) const
	{
		if (PropSkeletalMeshAsset != nullptr)
		{
			return PropSkeletalMeshAsset->GetMeshDescription(LodIndex);
		}
	
		if (PropStaticMeshAsset != nullptr)
		{
			return PropStaticMeshAsset->GetMeshDescription(LodIndex);
		}

		return nullptr;
	}
#endif //WITH_EDITOR
};

USTRUCT(BlueprintType, meta = (DisplayName = "Preview Prop Settings"))
struct FIKRetargetPreviewPropOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

#if WITH_EDITOR
	UE_API virtual USkeleton* GetSkeleton(const FName InPropertyName);
#endif //WITH_EDITOR

	// Attach bone source -> target name mapping 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source Target Mapping", meta=(ReinitializeOnEdit))
	TMap<FName,FName> AttachMapping;
	
	// Prop info to preview when debug drawing enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug", meta=(ReinitializeOnEdit))
	TArray<FPreviewPropsData> PreviewProps;

	// Debug Material for displaying props (must work w/ dynamic mesh, e.g. physics materials)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug", meta=(ReinitializeOnEdit))
	TObjectPtr<UMaterialInterface> DebugMaterial;

	// Prop info to preview when debug drawing enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bShowSourceProps = false;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Prop Debug Preview"))
struct FIKRetargetPreviewPropOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;
	
	UE_API virtual void AnimGraphPreUpdateMainThread(
		USkeletalMeshComponent& SourceMeshComponent,
		USkeletalMeshComponent& TargetMeshComponent) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;

	UPROPERTY()
	FIKRetargetPreviewPropOpSettings Settings;

private:
	FName ApplyBoneMap(FName SourceBone);

#if WITH_EDITOR
	void UpdateAnimSeqPlayhead(UAnimInstance* SourceAnimInstance);
	float ComputePropAnimPlayhead(float PropAnimLength, float PropAnimStartTime, bool bLooping) const;
	
	void GetAnimSeqFramePose(const UAnimSequenceBase* AnimSeq, double Time, TArray<FName>& OutBones, TArray<FTransform>& OutPose) const;
	
	static bool GetMeshVertTris(FPropMeshData& OutMeshData, const FPreviewPropsData& PreviewProp);
	
public:
	UE_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;

	virtual bool HasDebugDrawing() override { return true; };
	void UpdateSkinnedVertices(USkeletalMesh* SkeletalMeshAsset, FPropMeshData& PropMeshDataIn, const TArray<FMatrix44f>& RefToPoseMatrices);
	void GetRefToAnimPoseMatrices(UAnimSequence* InAnimation, float AnimPlayhead, USkeletalMesh* SkeletalMeshAsset, TArray<FMatrix44f>& OutRefToPose) const;

private:
	// Bone transforms for prop preview
	FDebugPropDrawInfo DebugPropInfo;
	// Prop mesh info for all meshes (created in Initialize)
	TArray<FPropMeshData> PropMeshData;
	
	static UE_API FCriticalSection DebugDataMutex;
	
	// Cache playing anim-seq/montage for grabbing a playhead
	TObjectPtr<const UAnimSequenceBase> CacheSourceCharacterAnim;
	float SourceAnimPlayhead = 0.0f;
	
	TStrongObjectPtr<UMaterialInstanceDynamic> PropDefaultMaterial = nullptr;
#endif
};

/* The blueprint/python API for editing BodyIntersectIK Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetPreviewPropController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current op settings as a struct.
	 * @return FIKRetargetBodyIntersectIKOp struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetPreviewPropOpSettings GetSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRetargetBodyIntersectIKOp struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetPreviewPropOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
