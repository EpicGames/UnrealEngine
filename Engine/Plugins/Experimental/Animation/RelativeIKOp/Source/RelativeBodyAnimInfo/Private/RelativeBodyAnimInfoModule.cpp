// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelativeBodyAnimInfoModule.h"

#define LOCTEXT_NAMESPACE "RelativeBodyAnimInfoModule"

void FRelativeBodyAnimInfoModule::StartupModule()
{
}

void FRelativeBodyAnimInfoModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FRelativeBodyAnimInfoModule, RelativeBodyAnimInfo)