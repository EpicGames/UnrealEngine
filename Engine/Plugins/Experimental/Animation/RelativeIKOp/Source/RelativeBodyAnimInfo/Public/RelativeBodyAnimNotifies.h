// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNotifies/AnimNotify.h"

#include "RelativeBodyAnimNotifies.generated.h"

#define UE_API RELATIVEBODYANIMINFO_API

/**
 * Base class to limit options for notify class types in RelativeBodyAnimModifier
 */
UCLASS(MinimalAPI)
class URelativeBodyAnimNotifyBase : public UAnimNotify
{
	GENERATED_BODY()
};


/**
 * Baked relative body anim-notify storing all body relationships in single notify at anim start
 */
UCLASS(MinimalAPI)
class URelativeBodyBakeAnimNotify : public URelativeBodyAnimNotifyBase
{
	GENERATED_BODY()
	
public:

	// TODO: Use physics asset (possibly from override) instead of mesh component?
	/**
	 * OnRelativeBodyDenseNotify interface for contact pair info
	 * 
	 * @param MeshComp The source SkeletalMeshComponent
	 * @param Animation The current animation
	 * @param NumSamplesIn The number of samples in baked data
	 * @param BodyPairsIn Array of body pairs
	 * @param BodyPairsSampleTimeIn Array of sample times
	 * @param BodyPairsLocalReferenceIn Array of per-body (pair), per-sample, local reference positions of closest body verts
	 * @param BodyPairsIsParentDominatesIn Array indicating if parent body should be allowed to move (one entery per-body-pair) 
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "Animation")
	UE_API void OnRelativeBodyDenseNotify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, int NumSamplesIn, const TMap<FName,FTransform> &OffsetTransformsForBonesIn, const TArray<FName> &BodyPairsIn, const TArray<float> &BodyPairsSampleTimeIn, const TArray<FVector3f> &BodyPairsLocalReferenceIn, const TArray<bool> &BodyPairsIsParentDominatesIn) const;

	virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override
	{
		// UE_LOGF(LogTemp, Log, "URelativeIKDenseAnimNotify Triggered: %d constraints", NumSamples);
		OnRelativeBodyDenseNotify(MeshComp, Animation, NumSamples, OffsetTransformsForBones, BodyPairs, BodyPairsSampleTime, BodyPairsLocalReference, bBodyPairsIsParentDominates);
	}
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	int NumSamples = 0;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TMap<FName,FTransform> OffsetTransformsForBones;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TArray<FName> BodyPairs;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TArray<float> BodyPairsSampleTime;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TArray<FVector3f> BodyPairsLocalReference;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TArray<bool> bBodyPairsIsParentDominates;
};

/**
 * Per frame body relationships notify (generates a very large amount of data)
 */
UCLASS(MinimalAPI, meta = (DisplayName = "RelativeBodyPerFrameAnimNotify"))
class URelativeBodyPerFrameAnimNotify : public URelativeBodyAnimNotifyBase
{
	GENERATED_BODY()

public:
	// TODO: Should the mesh be switched to physics asset and pass override if available?
	void SetInfo(USkeletalMesh* SkeletalMeshAsset_, FName body1_, FName body2_, FVector3f &loc1_, FVector3f &loc2_, bool IsParentDominates_)
	{
		SkeletalMeshAsset = SkeletalMeshAsset_;
		body1 = body1_;
		body2 = body2_;
		loc1 = loc1_;
		loc2 = loc2_;
		bIsParentDominates = IsParentDominates_;
	}
	
	/**
	 * OnRelativeBodyAnimNotify interface for contact pair info
	 * 
	 * @param SkeletalMesh The source SkeletalMesh contains the PhysicsAsset information
	 * @param Body1 The domain body of this contact pair
	 * @param Body2 The contact body of this contact pair
	 * @param Loc1 The local reference position of contact point in the domain body
	 * @param Loc2 The local reference position of contact point in the contact body
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "Animation")
	UE_API void OnRelativeBodyAnimNotify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, USkeletalMesh* SkeletalMesh, FName Body1, FName Body2, const FVector3f &Loc1, const FVector3f &Loc2, bool IsParentDominates) const;

	// /**
	//  * Restore baked local reference points to the source body's world position
	//  * 
	//  * @param VecIn1 The local reference position of contact point in the domain body
	//  * @param VecIn2 The local reference position of contact point in the contact body
	//  * @param VecOut1 The domain body's contact position of the source character in world space
	//  * @param VecOut2 The contact body's contact position of the source character in world space
	//  */
	// UFUNCTION(BlueprintCallable, Category = "Animation")
	// void ConvertLocalToGlobal(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FVector3f &VecIn1, const FVector3f &VecIn2, FVector3f &VecOut1, FVector3f &VecOut2) const;
	
	virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override
	{
		// UE_LOGF(LogTemp, Log, "URelativeBodyPerFrameAnimNotify Triggered: %d constraints", NumSamples);
		OnRelativeBodyAnimNotify(MeshComp, Animation, SkeletalMeshAsset, body1, body2, loc1, loc2, bIsParentDominates);
	}

protected:
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	FName body1;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	FName body2;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	FVector3f loc1;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	FVector3f loc2;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
    bool bIsParentDominates;
};


/**
 * Baked relative props anim-notify storing all props relationships in single notify at anim start
 */
UCLASS(MinimalAPI)
class URelativePropsBakeAnimNotify : public URelativeBodyAnimNotifyBase
{
	GENERATED_BODY()

public:

	// TODO: Use physics asset (possibly from override) instead of mesh component?
	/**
	 * OnRelativeBodyDenseNotify interface for contact pair info
	 *
	 * @param MeshComp The source SkeletalMeshComponent
	 * @param Animation The current animation
	 * @param NumSamplesIn The number of samples in baked data
	 * @param BodyPairsIn Array of body pairs
	 * @param BodyPairsSampleTimeIn Array of sample times
	 * @param BodyPairsLocalReferenceIn Array of per-body (pair), per-sample, local reference positions of closest body verts
	 * @param BodyPairsIsParentDominatesIn Array indicating if parent body should be allowed to move (one entery per-body-pair)
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "Animation")
	UE_API void OnRelativePropsDenseNotify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, int NumSamplesIn, const TMap<FName,FTransform> &OffsetTransformsForBonesIn, const TArray<FName>& PropsPairsIn, const TArray<float>& PropsPairsSampleTimeIn, const TArray<FVector3f>& PropsPairsLocalReferenceIn) const;

	virtual void Notify(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, const FAnimNotifyEventReference& EventReference) override
	{
		// UE_LOGF(LogTemp, Log, "URelativeIKDenseAnimNotify Triggered: %d constraints", NumSamples);
		OnRelativePropsDenseNotify(MeshComp, Animation, NumSamples, OffsetTransformsForBones, PropsPairs, PropsPairsSampleTime, PropsPairsLocalReference);
	}

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	int NumSamples = 0;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TMap<FName,FTransform> OffsetTransformsForBones;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TArray<FName> PropsPairs;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TArray<float> PropsPairsSampleTime;
	
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TArray<FVector3f> PropsPairsLocalReference;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Detail"))
	TArray<bool> bPropsPairsIsParentDominates;
};

#undef UE_API
