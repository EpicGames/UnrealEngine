// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelativeBodyAnimModifier.h"

#include "AnimPose.h"
#include "MeshDescription.h"
#include "RelativeBodyAnimNotifies.h"
#include "RelativeBodyUtils.h"
#include "SkeletalMeshAttributes.h"
#include "Animation/AnimSequence.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/ScopedSlowTask.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Spatial/PointHashGrid3.h"
#include "Templates/UniquePtr.h"

#define LOCTEXT_NAMESPACE "RelativeBodyAnimModifier"

void URelativeBodyAnimModifier::OnApply_Implementation(UAnimSequence* InAnimation)
{
	// TODO: Make log category for relative body utils?
	if (InAnimation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativeBodyAnimNotify. Reason: Invalid Animation");
		return;
	}

	if (!SkeletalMeshAsset)
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativeBodyAnimNotify. Reason: Invalid SkeletalMeshComponent");
		return;
	}

	if (!SkeletalMeshAsset->CloneMeshDescription(LODIndex, MeshDescription))
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativeBodyAnimNotify. Reason: Could not clone mesh description");
		return;
	}

	if (!NotifyClass || NotifyClass == URelativeBodyAnimNotifyBase::StaticClass())
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativeBodyAnimNotify. Reason: Must specify valid relative body notify subclass to create");
		return;
	}

	if (!PhysicsAssetOverride)
	{
		UE_LOGF(LogAnimation, Warning, "RelativeBodyAnimModifier: No physics asset override specified, using skeletal mesh physics asset!")
	}

	//Cache
	UPhysicsAsset* PhysicsAsset = GetPhysicsAsset();
	FReferenceSkeleton& RefSkeleton = SkeletalMeshAsset->GetRefSkeleton();

	if (!PhysicsAsset)
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativeBodyAnimNotify. Reason: Must specify valid physics asset");
		return;
	}

	// TODO: Move cached data out of modifier (or check if it's outdated and only rebuild in that case)
	CacheBodyDataForSourceMesh(CachedBodySourceData);
	
	const TArray<int32>& BodyIndicesParentBodyIndices = CachedBodySourceData.BodyIndicesParentBodyIndices;
	const TArray<bool>& BodyIndicesToIgnore = CachedBodySourceData.BodyIndicesToIgnore;
	const TArray<bool>& IsDomainBodyIndices = CachedBodySourceData.IsDomainBody;
	const TArray<int32>& BodyIndicesToBoneIndices = CachedBodySourceData.BodyIndicesToSourceBoneIndices;
	const TArray<int32>& BoneIndicesToBodyIndices = CachedBodySourceData.SourceBoneIndicesToBodyIndices;
	const TArray<TArray<int32>>& VertexIndicesInfluencedByBodyIndices = CachedBodySourceData.SourceVertexIndicesInfluencedByBodyIndices;

	const int32 NumBodies = BodyIndicesToBoneIndices.Num();
	const float HashGridCellSize = std::min(1.5f*ContactThreshold, 50.f); // the length of the cell size in the point hash gri

	// Process animation asset
	{
		const FAnimPoseEvaluationOptions AnimPoseEvalOptions{
			EAnimDataEvalType::Raw,
			true,
			false,
			false,
			nullptr,
			true,
			false
		};
		
		const float SequenceLength = InAnimation->GetPlayLength();
		const float SampleStep = 1.0f / static_cast<float>(SampleRate);
		const int SampleNum = FMath::TruncToInt(SequenceLength / SampleStep);
		
		URelativeBodyBakeAnimNotify* BakedAnimNotifyInfo = nullptr;
		
		bool bUseDenseNotifyInfo = NotifyClass->IsChildOf(URelativeBodyBakeAnimNotify::StaticClass());
		if (bUseDenseNotifyInfo)
		{
			FName TrackName("RelBody-<" + SkeletalMeshAsset->GetName() + ">");
			UE_LOGF(LogAnimation, Verbose, "TrackName %d: '%ls'.", GeneratedNotifyTracks.Num()-1, *TrackName.ToString());
			const bool bDoesTrackNameAlreadyExist = UAnimationBlueprintLibrary::IsValidAnimNotifyTrackName(InAnimation, TrackName);
			if (!bDoesTrackNameAlreadyExist)
			{
				UAnimationBlueprintLibrary::AddAnimationNotifyTrack(InAnimation, TrackName, FLinearColor::MakeRandomColor());
				GeneratedNotifyTracks.Add(TrackName);
			}
			
			BakedAnimNotifyInfo = Cast<URelativeBodyBakeAnimNotify>(UAnimationBlueprintLibrary::AddAnimationNotifyEvent(InAnimation, TrackName, 0., NotifyClass));
			if (!BakedAnimNotifyInfo)
			{
				return;
			}
			
			for (int32 BodyIndex1 = 0; BodyIndex1 < NumBodies; ++BodyIndex1)
			{
				if (BodyIndicesToIgnore[BodyIndex1])
				{
					continue;
				}
				
				FName SourceBodyBoneName = RefSkeleton.GetBoneName(BodyIndicesToBoneIndices[BodyIndex1]);
				int32 SourceBodyIdx = PhysicsAsset->FindBodyIndex(SourceBodyBoneName);
				if (SourceBodyIdx == INDEX_NONE)
				{
					continue;
				}
				FKShapeElem* ShapeElem1 = PhysicsAsset->SkeletalBodySetups[SourceBodyIdx]->AggGeom.GetElement(0);
				FTransform3f BodyTransform1(ShapeElem1->GetTransform());
				FVector3f ElementScale3D1 = FRelativeBodyPhysicsUtils::CalcReferenceShapeScale3D(ShapeElem1);
				BodyTransform1.SetScale3D(ElementScale3D1);
				BakedAnimNotifyInfo->OffsetTransformsForBones.Emplace(SourceBodyBoneName, BodyTransform1);
				
				for (int32 BodyIndex2 = BodyIndex1 + 1; BodyIndex2 < NumBodies; ++BodyIndex2)
				{
					if (BodyIndicesToIgnore[BodyIndex2]) continue;
					if (BodyIndicesParentBodyIndices[BodyIndex1] == BodyIndex2 || BodyIndicesParentBodyIndices[BodyIndex2] == BodyIndex1) continue;
					if (IsDomainBodyIndices[BodyIndex1]&&IsDomainBodyIndices[BodyIndex2]) continue;

					int32 QueryBodyIndex = VertexIndicesInfluencedByBodyIndices[BodyIndex1].Num() < VertexIndicesInfluencedByBodyIndices[BodyIndex2].Num() ? BodyIndex1 : BodyIndex2;
					int32 HashBodyIndex = VertexIndicesInfluencedByBodyIndices[BodyIndex1].Num() < VertexIndicesInfluencedByBodyIndices[BodyIndex2].Num() ? BodyIndex2 : BodyIndex1;
					bool IsParentDominates = false;
					if (IsDomainBodyIndices[BodyIndex1])
					{
						QueryBodyIndex = BodyIndex2;
						HashBodyIndex = BodyIndex1;
						IsParentDominates = true;
					}
					else if (IsDomainBodyIndices[BodyIndex2])
					{
						QueryBodyIndex = BodyIndex1;
						HashBodyIndex = BodyIndex2;
						IsParentDominates = true;
					}
					else if (VertexIndicesInfluencedByBodyIndices[BodyIndex1].Num() < VertexIndicesInfluencedByBodyIndices[BodyIndex2].Num())
					{
						QueryBodyIndex = BodyIndex1;
						HashBodyIndex = BodyIndex2;
					}
					else
					{
						QueryBodyIndex = BodyIndex2;
						HashBodyIndex = BodyIndex1;
					}
				
					FName BoneName1 = RefSkeleton.GetBoneName(BodyIndicesToBoneIndices[HashBodyIndex]);
					FName BoneName2 = RefSkeleton.GetBoneName(BodyIndicesToBoneIndices[QueryBodyIndex]);
					// FName BodyPairName("<" + SkeletalMeshAsset->GetName() + ">" + BoneName1.ToString() + "-" + BoneName2.ToString());
					BakedAnimNotifyInfo->BodyPairs.Add(BoneName1);
					BakedAnimNotifyInfo->BodyPairs.Add(BoneName2);
					BakedAnimNotifyInfo->bBodyPairsIsParentDominates.Add(IsParentDominates);
				}
			}
			
			BakedAnimNotifyInfo->BodyPairsLocalReference.SetNumZeroed(BakedAnimNotifyInfo->BodyPairs.Num()*SampleNum);
			BakedAnimNotifyInfo->BodyPairsSampleTime.SetNumZeroed(SampleNum);
			BakedAnimNotifyInfo->NumSamples = SampleNum;
		}

		FScopedSlowTask BakePairVertsTask((float)SampleNum, LOCTEXT("BakeVertsTaskText", "Baking Body Pair Verts..."));
		BakePairVertsTask.MakeDialog();

		// Get ground levels and max speed values.
		TArray<FVector3f> VLocations;
		for (int SampleIndex = 0; SampleIndex < SampleNum; ++SampleIndex)
		{
			BakePairVertsTask.EnterProgressFrame(1.0f);
			
			const float SampleTime = FMath::Clamp(static_cast<float>(SampleIndex) * SampleStep, 0.0f, SequenceLength);
			const float FutureSampleTime = FMath::Clamp((static_cast<float>(SampleIndex) + 1.0f) * SampleStep, 0.0f, SequenceLength);

			FAnimPose AnimPose;
			FAnimPose FutureAnimPose;

			UAnimPoseExtensions::GetAnimPoseAtTime(InAnimation, SampleTime, AnimPoseEvalOptions, AnimPose);
			UAnimPoseExtensions::GetAnimPoseAtTime(InAnimation, FutureSampleTime, AnimPoseEvalOptions, FutureAnimPose);

			TArray<FMatrix44f> CacheToLocals;
			GetRefToAnimPoseMatrices(CacheToLocals, AnimPose);
			GetSkinnedVertices(VLocations, CacheToLocals);

			// build a spatial hash grid
			TArray<TUniquePtr<UE::Geometry::TPointHashGrid3f<int32>>> BodiesVertHash;
			BodiesVertHash.SetNum(NumBodies);
			size_t count = 0;
			for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
			{
				BodiesVertHash[BodyIndex] = MakeUnique<UE::Geometry::TPointHashGrid3f<int32>>(HashGridCellSize, INDEX_NONE);
				BodiesVertHash[BodyIndex]->Reserve(VertexIndicesInfluencedByBodyIndices[BodyIndex].Num());
				for (int32 VertexIndex : VertexIndicesInfluencedByBodyIndices[BodyIndex])
				{
					BodiesVertHash[BodyIndex]->InsertPointUnsafe(VertexIndex, static_cast<FVector3f>(VLocations[VertexIndex]));
				}
			}

			TArray<FName> ContactedBodyPairs;
			if (BakedAnimNotifyInfo)
			{
				BakedAnimNotifyInfo->BodyPairsSampleTime[SampleIndex] = SampleTime;
			}
			for (int32 BodyIndex1 = 0; BodyIndex1 < NumBodies; ++BodyIndex1)
			{
				if (BodyIndicesToIgnore[BodyIndex1]) continue;
				for (int32 BodyIndex2 = BodyIndex1 + 1; BodyIndex2 < NumBodies; ++BodyIndex2)
				{
					if (BodyIndicesToIgnore[BodyIndex2]) continue;
					if (BodyIndicesParentBodyIndices[BodyIndex1] == BodyIndex2 || BodyIndicesParentBodyIndices[BodyIndex2] == BodyIndex1) continue;
					if (IsDomainBodyIndices[BodyIndex1]&&IsDomainBodyIndices[BodyIndex2]) continue;
					
					int32 QueryBodyIndex = VertexIndicesInfluencedByBodyIndices[BodyIndex1].Num() < VertexIndicesInfluencedByBodyIndices[BodyIndex2].Num() ? BodyIndex1 : BodyIndex2;
					int32 HashBodyIndex = VertexIndicesInfluencedByBodyIndices[BodyIndex1].Num() < VertexIndicesInfluencedByBodyIndices[BodyIndex2].Num() ? BodyIndex2 : BodyIndex1;
					bool IsParentDominates = false;
					if (IsDomainBodyIndices[BodyIndex1])
					{
						QueryBodyIndex = BodyIndex2;
						HashBodyIndex = BodyIndex1;
						IsParentDominates = true;
					}
					else if (IsDomainBodyIndices[BodyIndex2])
					{
						QueryBodyIndex = BodyIndex1;
						HashBodyIndex = BodyIndex2;
						IsParentDominates = true;
					}
					else if (VertexIndicesInfluencedByBodyIndices[BodyIndex1].Num() < VertexIndicesInfluencedByBodyIndices[BodyIndex2].Num())
					{
						QueryBodyIndex = BodyIndex1;
						HashBodyIndex = BodyIndex2;
					}
					else
					{
						QueryBodyIndex = BodyIndex2;
						HashBodyIndex = BodyIndex1;
					}

					float MinDist = FLT_MAX;
					int32 NearestHashVertexIndex = INDEX_NONE;
					int32 NearestQueryVertexIndex = INDEX_NONE;
					for (int32 QueryVertexIndex : VertexIndicesInfluencedByBodyIndices[QueryBodyIndex])
					{
						FVector3f QueryPoint = VLocations[QueryVertexIndex];
						TPair<int32, float> AnyClosePoint = { INDEX_NONE, TNumericLimits<float>::Max() };
						AnyClosePoint = BodiesVertHash[HashBodyIndex]->FindNearestInRadius(
							QueryPoint,
							ContactThreshold,
							[&VLocations, QueryPoint](int32 VID)
							{
								return FVector3f::DistSquared(FVector3f(VLocations[VID]), QueryPoint);
							});
						if (AnyClosePoint.Value < MinDist)
						{
							NearestQueryVertexIndex = QueryVertexIndex;
							NearestHashVertexIndex = AnyClosePoint.Key;
							MinDist = AnyClosePoint.Value;
						}
					}
					if (NearestQueryVertexIndex != INDEX_NONE)
					{
						FName BoneName1 = RefSkeleton.GetBoneName(BodyIndicesToBoneIndices[HashBodyIndex]);
						FName BoneName2 = RefSkeleton.GetBoneName(BodyIndicesToBoneIndices[QueryBodyIndex]);
						FVector3f ContactPoint1 = VLocations[NearestHashVertexIndex];
						FVector3f ContactPoint2 = VLocations[NearestQueryVertexIndex];
						ContactedBodyPairs.Add(BoneName1);
						ContactedBodyPairs.Add(BoneName2);

						FVector3f Loc1 = FRelativeBodyAnimUtils::CalcVertLocationInUnitBody(ContactPoint1, BoneName1, RefSkeleton, CacheToLocals, CachedBodySourceData.SourceRetargetGlobalPose, PhysicsAsset);
						FVector3f Loc2 = FRelativeBodyAnimUtils::CalcVertLocationInUnitBody(ContactPoint2, BoneName2, RefSkeleton, CacheToLocals, CachedBodySourceData.SourceRetargetGlobalPose, PhysicsAsset);
						
						// Use your custom notify class
						if (!bUseDenseNotifyInfo)
						{
							FName TrackName("<" + SkeletalMeshAsset->GetName() + ">" + BoneName1.ToString() + "-" + BoneName2.ToString());
						
							UE_LOGF(LogAnimation, Verbose, "TrackName %d: '%ls'.", GeneratedNotifyTracks.Num()-1, *TrackName.ToString());
							const bool bDoesTrackNameAlreadyExist = UAnimationBlueprintLibrary::IsValidAnimNotifyTrackName(InAnimation, TrackName);
							if (!bDoesTrackNameAlreadyExist)
							{
								UAnimationBlueprintLibrary::AddAnimationNotifyTrack(InAnimation, TrackName, FLinearColor::MakeRandomColor());
								GeneratedNotifyTracks.Add(TrackName);
							}
							
							URelativeBodyPerFrameAnimNotify* NewNotify = Cast<URelativeBodyPerFrameAnimNotify>(UAnimationBlueprintLibrary::AddAnimationNotifyEvent(InAnimation, TrackName, SampleTime, NotifyClass));
							if (NewNotify)
							{
								NewNotify->SetInfo(SkeletalMeshAsset,BoneName1,BoneName2,Loc1,Loc2,IsParentDominates);
							}
						}
						else if (BakedAnimNotifyInfo)
						{
							FTransform3f SourceBodyRot1(FTransform(BakedAnimNotifyInfo->OffsetTransformsForBones[BoneName1].GetRotation()));
							FTransform3f SourceBodyRot2(FTransform(BakedAnimNotifyInfo->OffsetTransformsForBones[BoneName2].GetRotation()));
							
							size_t offset = SampleIndex*BakedAnimNotifyInfo->BodyPairs.Num();
							BakedAnimNotifyInfo->BodyPairsLocalReference[offset+count*2+0] = SourceBodyRot1.TransformPosition(Loc1);
							BakedAnimNotifyInfo->BodyPairsLocalReference[offset+count*2+1] = SourceBodyRot2.TransformPosition(Loc2);
						}
					}
					count++;
				}
			}
		}
	}
}

void URelativeBodyAnimModifier::OnRevert_Implementation(UAnimSequence* InAnimation)
{
	// Delete any generate tracks.
	for (const FName GeneratedNotifyTrack : GeneratedNotifyTracks)
	{
		UAnimationBlueprintLibrary::RemoveAnimationNotifyTrack(InAnimation, GeneratedNotifyTrack);
	}

	GeneratedNotifyTracks.Reset();
}

UPhysicsAsset* URelativeBodyAnimModifier::GetPhysicsAsset() const
{
	if (PhysicsAssetOverride)
	{
		return PhysicsAssetOverride; 
	}
	
	if (SkeletalMeshAsset)
	{
		return SkeletalMeshAsset->GetPhysicsAsset();
	}
	
	return nullptr;
}

void URelativeBodyAnimModifier::CacheBodyDataForSourceMesh(FRelativeBodySourceData& OutSourceData)
{
	const int32 NumVertices = MeshDescription.Vertices().Num();
	
	//Extract skin weights
	UPhysicsAsset* PhysicsAsset = GetPhysicsAsset();
	const FReferenceSkeleton& RefSkeleton = SkeletalMeshAsset->GetRefSkeleton();
	FSkeletalMeshAttributes MeshAttribs(MeshDescription);
	FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();

	//CPU skinninged vertices
	const int32 NumBones = MeshAttribs.GetNumBones();
	OutSourceData.SourceVLocations.Empty();
	OutSourceData.SourceVLocations.AddUninitialized(NumVertices);
	OutSourceData.BodyIndicesToSourceBoneIndices.Empty();
	OutSourceData.SourceBoneIndicesToBodyIndices.Init(INDEX_NONE, NumBones);

	OutSourceData.SourceRetargetGlobalPose.Empty();
	FRelativeBodyAnimUtils::GetRefGlobalPos(OutSourceData.SourceRetargetGlobalPose, RefSkeleton);
	
	TArray<FMatrix44f> SourceCacheToLocals;
	FRelativeBodyAnimUtils::GetRefPoseToLocalMatrices(SourceCacheToLocals, SkeletalMeshAsset, LODIndex, OutSourceData.SourceRetargetGlobalPose);
	
	OutSourceData.SourceVLocations.Empty();
	OutSourceData.SourceVLocations.AddUninitialized(NumVertices);
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		FVertexBoneWeights BoneWeights = VertexSkinWeights.Get(FVertexID(VertexIndex));
		const int32 InfluenceCount = BoneWeights.Num();
		OutSourceData.SourceVLocations[VertexIndex] = FVector3f(0, 0, 0);

		int32 VertexMajorBoneIndex = INDEX_NONE;
		float VertexMajorBoneWeight = 0.;
		FName VertexMajorBoneName;
		for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
		{
			int32 InfluenceBoneIndex = BoneWeights[InfluenceIndex].GetBoneIndex();
			float InfluenceBoneWeight = BoneWeights[InfluenceIndex].GetWeight();
			FName InfluenceBoneName = RefSkeleton.GetBoneName(InfluenceBoneIndex);
			UE_LOGF(LogAnimation, Log, "Source %d: Bone '%ls'[%f].", VertexIndex, *InfluenceBoneName.ToString(), InfluenceBoneWeight);
			if (InfluenceBoneWeight > VertexMajorBoneWeight)
			{
				VertexMajorBoneIndex = InfluenceBoneIndex;
				VertexMajorBoneWeight = InfluenceBoneWeight;
				VertexMajorBoneName = InfluenceBoneName;
			}
			// TODO: Is this necessary here? The ref/inv-ref should be identity transform
			const FMatrix44f RefToLocal = SourceCacheToLocals[InfluenceBoneIndex];
			OutSourceData.SourceVLocations[VertexIndex] += RefToLocal.TransformPosition(MeshDescription.GetVertexPosition(VertexIndex)) * InfluenceBoneWeight;
		}
		const int32 ParentBodyIndex = PhysicsAsset->FindBodyIndex(VertexMajorBoneName);
		FName ParentBoneName;
		if (ParentBodyIndex == INDEX_NONE)
			ParentBoneName = FRelativeBodyAnimUtils::FindParentBodyBoneName(VertexMajorBoneName, RefSkeleton, PhysicsAsset);
		else
			ParentBoneName = PhysicsAsset->SkeletalBodySetups[ParentBodyIndex]->BoneName;
		const int32 ParentBoneIndex = RefSkeleton.FindBoneIndex(ParentBoneName);
		if (ParentBoneIndex == INDEX_NONE) continue;

		if (OutSourceData.SourceBoneIndicesToBodyIndices[ParentBoneIndex] == INDEX_NONE)
		{
			OutSourceData.SourceBoneIndicesToBodyIndices[ParentBoneIndex] = OutSourceData.BodyIndicesToSourceBoneIndices.Num();
			OutSourceData.BodyIndicesToSourceBoneIndices.Add(ParentBoneIndex);

			TArray<int32>& VertexIndicesInfluenced = OutSourceData.SourceVertexIndicesInfluencedByBodyIndices.AddDefaulted_GetRef();
			VertexIndicesInfluenced.Add(VertexIndex);
		}
		else {
			OutSourceData.SourceVertexIndicesInfluencedByBodyIndices[OutSourceData.SourceBoneIndicesToBodyIndices[ParentBoneIndex]].Add(VertexIndex);
		}
	}

	int32 NumBodies = OutSourceData.BodyIndicesToSourceBoneIndices.Num();
	OutSourceData.BodyIndicesParentBodyIndices.Empty();
	OutSourceData.BodyIndicesParentBodyIndices.SetNum(NumBodies);
	OutSourceData.BodyIndicesToIgnore.Empty();
	OutSourceData.IsDomainBody.Empty();
	OutSourceData.BodyIndicesToIgnore.SetNum(NumBodies);
	OutSourceData.IsDomainBody.SetNum(NumBodies);
	for (int32 idx = 0; idx < NumBodies; ++idx)
	{
		FName BoneName = RefSkeleton.GetBoneName(OutSourceData.BodyIndicesToSourceBoneIndices[idx]);
		FName ParentBoneName = FRelativeBodyAnimUtils::FindParentBodyBoneName(BoneName, RefSkeleton, PhysicsAsset);
		int32 BodyIndicesParentBoneIndex = RefSkeleton.FindBoneIndex(ParentBoneName);
		if (BodyIndicesParentBoneIndex != INDEX_NONE)
		{
			OutSourceData.BodyIndicesParentBodyIndices[idx] = OutSourceData.SourceBoneIndicesToBodyIndices[BodyIndicesParentBoneIndex];
		}
		else
		{
			OutSourceData.BodyIndicesParentBodyIndices[idx] = INDEX_NONE;
		}

		OutSourceData.BodyIndicesToIgnore[idx] = true;
		OutSourceData.IsDomainBody[idx] = false;
		UE_LOGF(LogAnimation, Log, "%d: Bone '%ls' [%d verts].", idx, *BoneName.ToString(), OutSourceData.SourceVertexIndicesInfluencedByBodyIndices[idx].Num());
	}
	for (FName BodyName : DomainBodyNames)
	{
		int32 BodyIndex = PhysicsAsset->FindBodyIndex(BodyName);
		if (BodyIndex != INDEX_NONE)
		{
			FName BoneName = PhysicsAsset->SkeletalBodySetups[BodyIndex]->BoneName;
			int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
			OutSourceData.BodyIndicesToIgnore[OutSourceData.SourceBoneIndicesToBodyIndices[BoneIndex]] = false;
			OutSourceData.IsDomainBody[OutSourceData.SourceBoneIndicesToBodyIndices[BoneIndex]] = true;
		}
	}
	for (FName BodyName : ContactBodyNames)
	{
		int32 BodyIndex = PhysicsAsset->FindBodyIndex(BodyName);
		if (BodyIndex != INDEX_NONE)
		{
			FName BoneName = PhysicsAsset->SkeletalBodySetups[BodyIndex]->BoneName;
			int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
			OutSourceData.BodyIndicesToIgnore[OutSourceData.SourceBoneIndicesToBodyIndices[BoneIndex]] = false;
		}
	}
}

void URelativeBodyAnimModifier::GetRefToAnimPoseMatrices(TArray<FMatrix44f>& OutRefToPose, const FAnimPose& AnimPose) const
{
	const FReferenceSkeleton& RefSkeleton = SkeletalMeshAsset->GetRefSkeleton();
	const TArray<FMatrix44f>& RefBasesInvMatrix = SkeletalMeshAsset->GetRefBasesInvMatrix();
	
	OutRefToPose.Init(FMatrix44f::Identity, RefBasesInvMatrix.Num());

	const FSkeletalMeshLODRenderData& LOD = SkeletalMeshAsset->GetResourceForRendering()->LODRenderData[LODIndex];
	const TArray<FBoneIndexType>* RequiredBoneSets[3] = { &LOD.ActiveBoneIndices, nullptr, NULL };
	for (int32 RequiredBoneSetIndex = 0; RequiredBoneSets[RequiredBoneSetIndex] != NULL; RequiredBoneSetIndex++)
	{
		const TArray<FBoneIndexType>& RequiredBoneIndices = *RequiredBoneSets[RequiredBoneSetIndex];
		for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndices.Num(); BoneIndex++)
		{
			const int32 ThisBoneIndex = RequiredBoneIndices[BoneIndex];
			if (RefBasesInvMatrix.IsValidIndex(ThisBoneIndex))
			{
				FName ThisBoneName = RefSkeleton.GetBoneName(ThisBoneIndex);
				const FTransform ThisBoneTransform = UAnimPoseExtensions::GetBonePose(AnimPose, ThisBoneName, EAnimPoseSpaces::World);
				OutRefToPose[ThisBoneIndex] = static_cast<FMatrix44f>(ThisBoneTransform.ToMatrixWithScale());
			}
		}
	}

	for (int32 ThisBoneIndex = 0; ThisBoneIndex < OutRefToPose.Num(); ++ThisBoneIndex)
	{
		OutRefToPose[ThisBoneIndex] = RefBasesInvMatrix[ThisBoneIndex] * OutRefToPose[ThisBoneIndex];
	}
}

void URelativeBodyAnimModifier::GetSkinnedVertices(TArray<FVector3f>& VLocations, const TArray<FMatrix44f>& RefToPoseMatrices)
{
	const int32 NumVertices = MeshDescription.Vertices().Num();
	VLocations.Empty();
	VLocations.AddUninitialized(NumVertices);

	//Extract skin weights
	FReferenceSkeleton& RefSkeleton = SkeletalMeshAsset->GetRefSkeleton();
	FSkeletalMeshAttributes MeshAttribs(MeshDescription);
	FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();

	//CPU skinned vertices
	const int32 NumBones = MeshAttribs.GetNumBones();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		FVertexBoneWeights BoneWeights = VertexSkinWeights.Get(FVertexID(VertexIndex));
		const int32 InfluenceCount = BoneWeights.Num();
		VLocations[VertexIndex] = FVector3f(0, 0, 0);
		for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
		{
			int32 InfluenceBoneIndex = BoneWeights[InfluenceIndex].GetBoneIndex();
			float InfluenceBoneWeight = BoneWeights[InfluenceIndex].GetWeight();

			const FMatrix44f& RefToPose = RefToPoseMatrices[InfluenceBoneIndex];
			VLocations[VertexIndex] += RefToPose.TransformPosition(MeshDescription.GetVertexPosition(VertexIndex)) * InfluenceBoneWeight;
		}
	}
}

#undef LOCTEXT_NAMESPACE
