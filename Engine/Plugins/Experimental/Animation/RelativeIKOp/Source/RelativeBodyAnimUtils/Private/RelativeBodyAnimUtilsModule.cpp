// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelativeBodyAnimUtilsModule.h"

#define LOCTEXT_NAMESPACE "RelativeBodyAnimUtilsModule"

void FRelativeBodyAnimUtilsModule::StartupModule()
{
}

void FRelativeBodyAnimUtilsModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FRelativeBodyAnimUtilsModule, RelativeBodyAnimUtils)