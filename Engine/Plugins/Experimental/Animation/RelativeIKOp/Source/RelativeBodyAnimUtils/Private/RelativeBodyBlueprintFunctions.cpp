// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelativeBodyBlueprintFunctions.h"
#include "AnimationModifiersAssetUserData.h"
#include "RelativeBodyAnimNotifies.h"
#include "Animation/AnimSequence.h"
#include "Misc/ScopedSlowTask.h"

#define LOCTEXT_NAMESPACE "RelativeBodyBlueprintFunctions"

template <typename TRelativeBodyClass>
TRelativeBodyClass* FindRelativeBodyModifier(UAnimationModifiersAssetUserData* AssetUserData)
{
	const TArray<UAnimationModifier*>& ModifierInstances = AssetUserData->GetAnimationModifierInstances();
	for (UAnimationModifier* Modifier : ModifierInstances)
	{
		if (TRelativeBodyClass* CheckInst = Cast<TRelativeBodyClass>(Modifier))
		{
			return CheckInst;
		}
	}

	return nullptr;
}

template <typename TRelativeBodyClass>
TRelativeBodyClass* FindExactTypeBodyModifier(UAnimationModifiersAssetUserData* AssetUserData)
{
	const TArray<UAnimationModifier*>& ModifierInstances = AssetUserData->GetAnimationModifierInstances();
	for (UAnimationModifier* Modifier : ModifierInstances)
	{
		if (TRelativeBodyClass* CheckInst = ExactCast<TRelativeBodyClass>(Modifier))
		{
			return CheckInst;
		}
	}

	return nullptr;
}

void URelativeBodyAnimBlueprintFunctions::SetPropAttachBone(UPARAM(ref) FPropsInfo& Info, FName AttachBoneName)
{
	Info.SocketName = AttachBoneName;
}

FName URelativeBodyAnimBlueprintFunctions::GetPropAttachBone(const FPropsInfo& Info)
{
	return Info.SocketName;
}

float URelativeBodyAnimBlueprintFunctions::GetSegmentStartTime(const FAnimNotifyEvent& AnimNotifyEvent)
{
	return AnimNotifyEvent.GetTime();
}

void URelativeBodyAnimBlueprintFunctions::SetRelativeAnimModifiers(UPARAM(ref) TArray<UAnimSequenceBase*>& AnimationList, const FPropBodyAnimModifierOptions& Options)
{
	for (UAnimSequenceBase* Anim : AnimationList)
	{
		UAnimationModifiersAssetUserData* AssetUserData = Anim->GetAssetUserData<UAnimationModifiersAssetUserData>();
		if (!AssetUserData)
		{
			UE_LOGF(LogAnimation, Warning, "Unable to find modifier user data for anim: %ls", *Anim->GetName());
			continue;
		}

		TObjectPtr<URelativePropsAnimModifier> UpdateModifierInst = FindRelativeBodyModifier<URelativePropsAnimModifier>(AssetUserData);
		if (!UpdateModifierInst)
		{
			if (!AssetUserData->AddAnimationModifierOfClass(Anim, URelativePropsAnimModifier::StaticClass()))
			{
				UE_LOGF(LogAnimation, Warning, "Unable to create new modifier for anim: %ls", *Anim->GetName());
				continue;
			}

			UpdateModifierInst = FindRelativeBodyModifier<URelativePropsAnimModifier>(AssetUserData);
		}

		if (!UpdateModifierInst)
		{
			continue;
		}
		
		UpdateModifierInst->PropsData.Empty(Options.PropMeshSettings.Num());
		for (const FPropsInfo& PropInfo : Options.PropMeshSettings)
		{
			UpdateModifierInst->PropsData.Add(PropInfo);
		}
		UpdateModifierInst->bPropsFloorInfoBaking = Options.bPropsFloorInfoBaking;
		UpdateModifierInst->PropsNotifyClass = Options.PropNotifyClass;
		
		// Always set these
		UpdateModifierInst->SampleRate = Options.SampleRate;
		UpdateModifierInst->ContactThreshold = Options.ContactThreshold;
		UpdateModifierInst->SkeletalMeshAsset = Options.SkeletalMeshAsset;
		UpdateModifierInst->PhysicsAssetOverride = Options.PhysicsAssetOverride;
		UpdateModifierInst->DomainBodyNames = Options.DomainBodyNames;
		UpdateModifierInst->ContactBodyNames = Options.ContactBodyNames;
		UpdateModifierInst->NotifyClass = nullptr;
		
		// TODO: Ideally we would just delete the old body modifier in this case
		bool bUpdateSeparateBodyModifier = false;
		TObjectPtr<URelativeBodyAnimModifier> BodyModifierInst = FindExactTypeBodyModifier<URelativeBodyAnimModifier>(AssetUserData);
		if (Options.NotifyClass != nullptr && BodyModifierInst != nullptr)
		{
			// Update body modifier to generate the 
			bUpdateSeparateBodyModifier = true;
			BodyModifierInst->SampleRate = Options.SampleRate;
			BodyModifierInst->ContactThreshold = Options.ContactThreshold;
			BodyModifierInst->SkeletalMeshAsset = Options.SkeletalMeshAsset;
			BodyModifierInst->PhysicsAssetOverride = Options.PhysicsAssetOverride;
			BodyModifierInst->DomainBodyNames = Options.DomainBodyNames;
			BodyModifierInst->ContactBodyNames = Options.ContactBodyNames;
			BodyModifierInst->NotifyClass = Options.NotifyClass;
		}
		else
		{
			UpdateModifierInst->NotifyClass = Options.NotifyClass;
		}
	}
}

void URelativeBodyAnimBlueprintFunctions::BulkUpdateRelativeBodyModifiers(TArray<UAnimSequenceBase*>& AnimationList, const FRelativeBodyAnimModifierOptions& Options)
{
	FScopedSlowTask ApplyModifiersTask(static_cast<float>(AnimationList.Num()), LOCTEXT("BulkUpdateProgress", "Updating and applying modifier to animation list"));
	ApplyModifiersTask.MakeDialog();
	for (UAnimSequenceBase* Anim : AnimationList)
	{
		ApplyModifiersTask.EnterProgressFrame(1.0f, FText::FromString(Anim->GetName()));
		UAnimationModifiersAssetUserData* AssetUserData = Anim->GetAssetUserData<UAnimationModifiersAssetUserData>();
		if (!AssetUserData)
		{
			UE_LOGF(LogAnimation, Warning, "Unable to find modifier user data for anim: %ls", *Anim->GetName());
			continue;
		}

		TObjectPtr<URelativeBodyAnimModifier> UpdateModifierInst = FindRelativeBodyModifier<URelativeBodyAnimModifier>(AssetUserData);
		if (!UpdateModifierInst)
		{
			if (!AssetUserData->AddAnimationModifierOfClass(Anim, URelativeBodyAnimModifier::StaticClass()))
			{
				UE_LOGF(LogAnimation, Warning, "Unable to create new modifier for anim: %ls", *Anim->GetName());
				continue;
			}

			UpdateModifierInst = FindRelativeBodyModifier<URelativeBodyAnimModifier>(AssetUserData);
		}

		if (!UpdateModifierInst)
		{
			continue;
		}

		UpdateModifierInst->SampleRate = Options.SampleRate;
		UpdateModifierInst->ContactThreshold = Options.ContactThreshold;
		UpdateModifierInst->SkeletalMeshAsset = Options.SkeletalMeshAsset;
		UpdateModifierInst->PhysicsAssetOverride = Options.PhysicsAssetOverride;
		UpdateModifierInst->DomainBodyNames = Options.DomainBodyNames;
		UpdateModifierInst->ContactBodyNames = Options.ContactBodyNames;
		UpdateModifierInst->NotifyClass = Options.NotifyClass;

		UAnimSequence* AnimSeq = Cast<UAnimSequence>(Anim);
		if (AnimSeq)
		{
			UpdateModifierInst->ApplyToAnimationSequence(AnimSeq);
		}
	}
}

void URelativeBodyAnimBlueprintFunctions::BulkUpdatePropBodyModifiers(UPARAM(ref) TArray<UAnimSequenceBase*>& AnimationList, const FPropBodyAnimModifierOptions& Options)
{
	FScopedSlowTask ApplyModifiersTask(static_cast<float>(AnimationList.Num()), LOCTEXT("BulkUpdatePropProgress", "Updating and applying prop/body modifiers to animation list"));
	ApplyModifiersTask.MakeDialog();
	SetRelativeAnimModifiers(AnimationList, Options);
	
	for (UAnimSequenceBase* Anim : AnimationList)
	{
		ApplyModifiersTask.EnterProgressFrame(1.0f, FText::FromString(Anim->GetName()));
		UAnimationModifiersAssetUserData* AssetUserData = Anim->GetAssetUserData<UAnimationModifiersAssetUserData>();
		if (!AssetUserData)
		{
			UE_LOGF(LogAnimation, Warning, "Unable to find modifier user data for anim: %ls", *Anim->GetName());
			continue;
		}

		TObjectPtr<URelativePropsAnimModifier> PropModifierInst = FindRelativeBodyModifier<URelativePropsAnimModifier>(AssetUserData);
		TObjectPtr<URelativeBodyAnimModifier> BodyModifierInst = FindExactTypeBodyModifier<URelativeBodyAnimModifier>(AssetUserData);
		if (!PropModifierInst && !BodyModifierInst)
		{
			continue;
		}

		UAnimSequence* AnimSeq = Cast<UAnimSequence>(Anim);
		if (!AnimSeq)
		{
			continue;
		}
		
		if (PropModifierInst)
		{
			PropModifierInst->ApplyToAnimationSequence(AnimSeq);
		}
		if (BodyModifierInst)
		{
			BodyModifierInst->ApplyToAnimationSequence(AnimSeq);
		}
	}
}

#undef LOCTEXT_NAMESPACE
