// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelativeBodyUtils.h"

#include "Engine/SkeletalMesh.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Rendering/SkeletalMeshRenderData.h"

FVector3f FRelativeBodyPhysicsUtils::CalcReferenceShapeScale3D(FKShapeElem* ShapeElem)
{
	FVector3f ElementScale3D = FVector3f::One();
	switch (ShapeElem->GetShapeType())
	{
	case EAggCollisionShape::Sphere:
		{
			FKSphereElem* SphereElem = static_cast<FKSphereElem*>(ShapeElem);
			ElementScale3D = FVector3f(SphereElem->Radius);
			break;
		}

	case EAggCollisionShape::Box:
		{
			FKBoxElem* BoxElem = static_cast<FKBoxElem*>(ShapeElem);
			ElementScale3D = FVector3f(BoxElem->X, BoxElem->Y, BoxElem->Z);
			break;
		}

	case EAggCollisionShape::Sphyl:
		{
			FKSphylElem* CapsuleElem = static_cast<FKSphylElem*>(ShapeElem);
			ElementScale3D.X = CapsuleElem->Radius;
			ElementScale3D.Y = CapsuleElem->Radius;
			ElementScale3D.Z = CapsuleElem->Length * 0.5f + CapsuleElem->Radius;
			break;
		}

	case EAggCollisionShape::Convex:
		{
			// UE_LOGF(LogAnimation, Warning, "Unsupported: Shape is a Convex");
			break;
		}
	default:
		{
			// UE_LOGF(LogPhysicalIKRetargeter, Warning, "Unknown or unsupported shape type");
			break;
		}
	}
	ensure(!FMath::IsNearlyZero(ElementScale3D.X) && !FMath::IsNearlyZero(ElementScale3D.Y) && !FMath::IsNearlyZero(ElementScale3D.Z));
	
	return ElementScale3D;
}


FName FRelativeBodyAnimUtils::FindParentBodyBoneName(const FName BoneName, const FReferenceSkeleton& RefSkeleton, const UPhysicsAsset* PhysicsAsset)
{
	if (!PhysicsAsset)
	{
		return NAME_None;
	}
	
	const int32 StartBoneIndex = RefSkeleton.FindBoneIndex(BoneName);
	if (StartBoneIndex == INDEX_NONE)
	{
		return NAME_None;
	}
	
	const int32 ParentBodyIndex = PhysicsAsset->FindParentBodyIndex(RefSkeleton, StartBoneIndex);
    if (!PhysicsAsset->SkeletalBodySetups.IsValidIndex(ParentBodyIndex) || !PhysicsAsset->SkeletalBodySetups[ParentBodyIndex])
    {
    	return NAME_None;
    }

	return PhysicsAsset->SkeletalBodySetups[ParentBodyIndex]->BoneName;;
}

void FRelativeBodyAnimUtils::GetRefGlobalPos(TArray<FTransform>& GlobalRefPose, const FReferenceSkeleton& RefSkeleton)
{
	// record the name of the retarget pose (prevents re-initialization if profile swaps it)
	TArray<FTransform> RetargetLocalPose = RefSkeleton.GetRefBonePose();
	GlobalRefPose = RetargetLocalPose;

	int32 NumBones = RefSkeleton.GetNum();

	// convert to global space
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
		if (ParentIndex == INDEX_NONE)
		{
			// root always in global space already, no conversion required
			GlobalRefPose[BoneIndex] = RetargetLocalPose[BoneIndex];
			continue;
		}
		const FTransform& ChildLocalTransform = RetargetLocalPose[BoneIndex];
		const FTransform& ParentGlobalTransform = GlobalRefPose[ParentIndex];
		GlobalRefPose[BoneIndex] = ChildLocalTransform * ParentGlobalTransform;
	}

	// strip scale (done AFTER generating global pose so that scales are baked into translation)
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		RetargetLocalPose[BoneIndex].SetScale3D(FVector::OneVector);
		GlobalRefPose[BoneIndex].SetScale3D(FVector::OneVector);
	}
}

void FRelativeBodyAnimUtils::GetRefPoseToLocalMatrices(TArray<FMatrix44f>& OutRefToLocals, TObjectPtr<USkeletalMesh> SkeletalMeshAsset, int32 LODIndex, const TArray<FTransform>& RetargetGlobalPose)
{
	const TArray<FMatrix44f>& RefBasesInvMatrix = SkeletalMeshAsset->GetRefBasesInvMatrix();
	OutRefToLocals.Init(FMatrix44f::Identity, RefBasesInvMatrix.Num());

	const FSkeletalMeshLODRenderData& LOD = SkeletalMeshAsset->GetResourceForRendering()->LODRenderData[LODIndex];
	const TArray<FBoneIndexType>* RequiredBoneSets[3] = { &LOD.ActiveBoneIndices, nullptr, NULL };
	for (int32 RequiredBoneSetIndex = 0; RequiredBoneSets[RequiredBoneSetIndex] != NULL; RequiredBoneSetIndex++)
	{
		const TArray<FBoneIndexType>& RequiredBoneIndices = *RequiredBoneSets[RequiredBoneSetIndex];
		for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndices.Num(); BoneIndex++)
		{
			const int32 ThisBoneIndex = RequiredBoneIndices[BoneIndex];
			if (RefBasesInvMatrix.IsValidIndex(ThisBoneIndex))
			{
				OutRefToLocals[ThisBoneIndex] = FMatrix44f::Identity;
				if (RetargetGlobalPose.IsValidIndex(ThisBoneIndex))
				{
					OutRefToLocals[ThisBoneIndex] = static_cast<FMatrix44f>(RetargetGlobalPose[ThisBoneIndex].ToMatrixWithScale());
				}
			}
		}
	}

	for (int32 ThisBoneIndex = 0; ThisBoneIndex < OutRefToLocals.Num(); ++ThisBoneIndex)
	{
		OutRefToLocals[ThisBoneIndex] = RefBasesInvMatrix[ThisBoneIndex] * OutRefToLocals[ThisBoneIndex];
	}
}

FVector3f FRelativeBodyAnimUtils::CalcVertLocationInUnitBody(const FVector3f& RefPoint, const FName BodyBoneName, const FReferenceSkeleton& SourceRefSkeleton,  const TArray<FMatrix44f>& CacheToLocals, const TArray<FTransform>& SourceGlobalPose, UPhysicsAsset* SourcePhysicsAsset)
{
	int32 SourceBodyIdx = SourcePhysicsAsset->FindBodyIndex(BodyBoneName);
	FName SourceBodyBoneName = BodyBoneName;
	if (SourceBodyIdx == INDEX_NONE)
	{
		SourceBodyBoneName = FindParentBodyBoneName(BodyBoneName, SourceRefSkeleton, SourcePhysicsAsset);
		SourceBodyIdx = SourcePhysicsAsset->FindBodyIndex(SourceBodyBoneName);
	}
	int32 SourceBoneIndex = SourceRefSkeleton.FindBoneIndex(SourceBodyBoneName);
	FKShapeElem* ShapeElem = SourcePhysicsAsset->SkeletalBodySetups[SourceBodyIdx]->AggGeom.GetElement(0);
	FTransform3f SourceBoneTransform(SourceGlobalPose[SourceBoneIndex]);
	FTransform3f SourceBodyTransform(ShapeElem->GetTransform());
	FVector3f ElementScale3D = FRelativeBodyPhysicsUtils::CalcReferenceShapeScale3D(ShapeElem);

	FTransform3f CacheToLocalsTransform(CacheToLocals[SourceBoneIndex]);
	FVector3f SourcePoint = (SourceBodyTransform * SourceBoneTransform).InverseTransformPosition(
																			CacheToLocalsTransform.InverseTransformPosition(RefPoint));
	SourcePoint = SourcePoint / ElementScale3D;
	return SourcePoint;
}