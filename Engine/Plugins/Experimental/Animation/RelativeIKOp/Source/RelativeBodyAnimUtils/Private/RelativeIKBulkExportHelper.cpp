// Copyright Epic Games, Inc. All Rights Reserved.
#include "RelativeIKBulkExportHelper.h"

#include "Animation/AnimSequence.h"
#include "Retargeter/IKRetargeter.h"
#include "RetargetEditor/IKRetargetBatchOperation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RelativeIKBulkExportHelper)

void URelativeIKBulkExportHelper::BulkRetargetSequences(
	UIKRetargeter* RetargetAsset,
	UPARAM(ref) const TArray<UAnimSequence*>& RetargetSequenceList,
	const FString& OutPath,
	USkeletalMesh* SourceMesh,
	USkeletalMesh* TargetMesh)
{
	if (RetargetSequenceList.IsEmpty())
	{
		return;
	}
	
	FIKRetargetBatchOperationContext BatchContext;
	BatchContext.NameRule.FolderPath = OutPath;
	BatchContext.SourceMesh = SourceMesh;
	BatchContext.TargetMesh = TargetMesh;
	BatchContext.IKRetargetAsset = RetargetAsset;

	// BatchContext.NameRule.Prefix = Prefix;
	// BatchContext.NameRule.Suffix = TEXT("");
	// BatchContext.NameRule.ReplaceFrom = SearchRM;
	// BatchContext.NameRule.ReplaceTo = TEXT("");

	BatchContext.bOverwriteExistingFiles = true;

	for (const TObjectPtr<UAnimSequence> Seq : RetargetSequenceList)
	{
		if (!Seq)
		{
			continue;
		}
		
		BatchContext.AssetsToRetarget.Add(Seq);
	}

	const TStrongObjectPtr<UIKRetargetBatchOperation> BatchOperation(NewObject<UIKRetargetBatchOperation>());
	BatchOperation->RunRetarget(BatchContext);
}
