// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelativePropsAnimModifier.h"

#include "AnimPose.h"
#include "MeshDescription.h"
#include "RelativeBodyAnimModifier.h"
#include "RelativeBodyAnimNotifies.h"
#include "RelativeBodyUtils.h"
#include "SkeletalMeshAttributes.h"
#include "Animation/AnimSequence.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/ScopedSlowTask.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Spatial/PointHashGrid3.h"

#include "StaticMeshAttributes.h"
#include "MeshQueries.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "MeshAdapter.h"
#include "Spatial/MeshAABBTree3.h"
#include "Algo/MinElement.h"
#include "Engine/SkeletalMeshSocket.h"

#define LOCTEXT_NAMESPACE "RelativePropsAnimModifier"

int32 FindLowestIndex(const TArray<FVector3f>& Points)
{
	if (Points.Num() == 0)
	{
		return INDEX_NONE;
	}

	auto It = Algo::MinElement(
		Points,
		[](const FVector3f& A, const FVector3f& B)
		{
			return A.Z < B.Z;
		}
	);

	return It ? UE_PTRDIFF_TO_INT32(It - Points.GetData()) : INDEX_NONE;
}

int32 FindHighestIndex(const TArray<FVector3f>& Points)
{
	if (Points.Num() == 0)
	{
		return INDEX_NONE;
	}

	auto It = Algo::MinElement(
		Points,
		[](const FVector3f& A, const FVector3f& B)
		{
			return A.Z > B.Z;
		}
	);

	return It ? UE_PTRDIFF_TO_INT32(It - Points.GetData()) : INDEX_NONE;
}

struct FPropsMeshAndTree
{
	UE::Geometry::FDynamicMesh3 Mesh;
	UE::Geometry::TMeshAABBTree3<UE::Geometry::FDynamicMesh3> AABBTree;
	TArray<int32> DynVidToSourceVid;  // Maps DynamicMesh VID → MeshDesc VID

	FPropsMeshAndTree()
		: AABBTree(&Mesh, false) {}  // defer build

	void Init(const FMeshDescription* MeshDesc)
	{
		FMeshDescriptionToDynamicMesh Converter;
		Converter.Convert(MeshDesc, Mesh);
		// Build mapping: identity for original vertices,
		// source vertex for split (bowtie) vertices.
		// FMeshDescriptionToDynamicMesh internally tracks this mapping
		// during non-manifold topology repair.
		int32 MaxVid = Mesh.MaxVertexID();
		int32 OriginalCount = MeshDesc->Vertices().Num();
		DynVidToSourceVid.SetNumUninitialized(MaxVid);
		for (int32 Vid : Mesh.VertexIndicesItr())
		{
			// Original vertices map to themselves; split vertices
			// share the same position as their source — find by position match
			// or use the converter's internal VtxMap if accessible.
			DynVidToSourceVid[Vid] = (Vid < OriginalCount) ? Vid : INDEX_NONE;
		}
		// For split vertices (Vid >= OriginalCount), find closest original vertex
		// by position to establish the mapping.
		for (int32 Vid : Mesh.VertexIndicesItr())
		{
			if (Vid >= OriginalCount)
			{
				FVector3d SplitPos = Mesh.GetVertex(Vid);
				for (int32 OrigVid = 0; OrigVid < OriginalCount; ++OrigVid)
				{
					if (Mesh.GetVertex(OrigVid).Equals(SplitPos, SMALL_NUMBER))
					{
						DynVidToSourceVid[Vid] = OrigVid;
						break;
					}
				}
			}
		}
		AABBTree.Build();
	}
	
	// Update all vertex positions with a lambda
	void UpdateVertexPositions(TArray<FVector3f>& VLocation)
	{
		for (int32 Vid : Mesh.VertexIndicesItr())
		{
			int32 Src = (Vid < DynVidToSourceVid.Num()) ? DynVidToSourceVid[Vid] : Vid;
			if (Src >= 0 && Src < VLocation.Num())
			{
				Mesh.SetVertex(Vid, FVector3d(VLocation[Src]));
			}
		}
		AABBTree.Build();
	}
};

const FPositionVertexBuffer& FPropsInfo::GetPositionVertexBuffer()
{
	if (PropSkeletalMeshAsset != nullptr)
	{
		const FSkeletalMeshRenderData* RenderData = PropSkeletalMeshAsset->GetResourceForRendering();
		const FSkeletalMeshLODRenderData& LODData = RenderData->LODRenderData[0];
		return LODData.StaticVertexBuffers.PositionVertexBuffer;
	}

	if (PropStaticMeshAsset != nullptr)
	{
		const FStaticMeshLODResources& LODResources = PropStaticMeshAsset->GetRenderData()->LODResources[0];
		return LODResources.VertexBuffers.PositionVertexBuffer;
	}

	// Fallback: return an empty (static) vertex buffer to avoid crashes
	static FPositionVertexBuffer EmptyBuffer;
	return EmptyBuffer;
}

FMeshDescription* FPropsInfo::GetMeshDescription(int32 LodIndex)
{
	if (PropSkeletalMeshAsset != nullptr)
	{
		return PropSkeletalMeshAsset->GetMeshDescription(LodIndex);
	}
	
	if (PropStaticMeshAsset != nullptr)
	{
		return PropStaticMeshAsset->GetMeshDescription(LodIndex);
	}

	return nullptr;
}

float FPropsInfo::ComputePropAnimPlayhead(float CharacterPlayhead) const
{
	if (!PropAnimSequence)
	{
		return -1.f;
	}
	
	if (!PropAnimIsLooping)
	{
		return FMath::Min(PropAnimSequence->GetPlayLength(), CharacterPlayhead-PropAnimStartTime);
	}
	
	return FMath::Fmod(CharacterPlayhead-PropAnimStartTime, PropAnimSequence->GetPlayLength());
}

void FPropsInfo::GetSkinnedVertices(const TArray<FMatrix44f>& RefToPoseMatrices, TArray<FVector3f>& VLocation)
{
	if (!PropSkeletalMeshAsset)
	{
		return;
	}
	FMeshDescription* MeshDescriptionPtr = PropSkeletalMeshAsset->GetMeshDescription(0);
	if (!MeshDescriptionPtr)
	{
		return;
	}
	const int32 NumVertices = MeshDescriptionPtr->Vertices().Num();

	//Extract skin weights
	FReferenceSkeleton& RefSkeleton = PropSkeletalMeshAsset->GetRefSkeleton();
	FSkeletalMeshAttributes MeshAttribs(*MeshDescriptionPtr);
	FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();
	
	//CPU skinned vertices
	const int32 NumBones = MeshAttribs.GetNumBones();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		FVector3f NewPosition = FVector3f::ZeroVector;
		FVertexBoneWeights BoneWeights = VertexSkinWeights.Get(FVertexID(VertexIndex));
		const int32 InfluenceCount = BoneWeights.Num();
		for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
		{
			int32 InfluenceBoneIndex = BoneWeights[InfluenceIndex].GetBoneIndex();
			float InfluenceBoneWeight = BoneWeights[InfluenceIndex].GetWeight();

			const FMatrix44f& RefToPose = RefToPoseMatrices[InfluenceBoneIndex];
			NewPosition += RefToPose.TransformPosition(MeshDescriptionPtr->GetVertexPosition(VertexIndex)) * InfluenceBoneWeight;
		}
		VLocation[VertexIndex] = NewPosition;
	}
}

void FPropsInfo::GetRefToAnimPoseMatrices(float AnimPlayhead, TArray<FMatrix44f>& OutRefToPose) const
{
	if (!PropAnimSequence || !PropSkeletalMeshAsset)
	{
		return;
	}
	
	static FAnimPoseEvaluationOptions AnimPoseEvalOptions{
		EAnimDataEvalType::Raw,
		true,
		false,
		false,
		nullptr,
		true,
		false
	};
	
	FAnimPose AnimPose;
	UAnimPoseExtensions::GetAnimPoseAtTime(PropAnimSequence, AnimPlayhead, AnimPoseEvalOptions, AnimPose);
	const FReferenceSkeleton& RefSkeleton = PropSkeletalMeshAsset->GetRefSkeleton();
	const TArray<FMatrix44f>& RefBasesInvMatrix = PropSkeletalMeshAsset->GetRefBasesInvMatrix();
	
	OutRefToPose.Init(FMatrix44f::Identity, RefBasesInvMatrix.Num());
	
	const FSkeletalMeshLODRenderData& LOD = PropSkeletalMeshAsset->GetResourceForRendering()->LODRenderData[0];
	const TArray<FBoneIndexType>& RequiredBoneIndices = LOD.ActiveBoneIndices;
	for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndices.Num(); BoneIndex++)
	{
		const int32 ThisBoneIndex = RequiredBoneIndices[BoneIndex];
		if (RefBasesInvMatrix.IsValidIndex(ThisBoneIndex))
		{
			FName ThisBoneName = RefSkeleton.GetBoneName(ThisBoneIndex);
			const FTransform ThisBoneTransform = UAnimPoseExtensions::GetBonePose(AnimPose, ThisBoneName, EAnimPoseSpaces::World);
			OutRefToPose[ThisBoneIndex] = static_cast<FMatrix44f>(ThisBoneTransform.ToMatrixWithScale());
		}
	}

	for (int32 ThisBoneIndex = 0; ThisBoneIndex < OutRefToPose.Num(); ++ThisBoneIndex)
	{
		OutRefToPose[ThisBoneIndex] = RefBasesInvMatrix[ThisBoneIndex] * OutRefToPose[ThisBoneIndex];
	}
}

void URelativePropsAnimModifier::OnApply_Implementation(UAnimSequence* InAnimation)
{
	// TODO: Make log category for relative body utils?
	if (InAnimation == nullptr)
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativeBodyAnimNotify. Reason: Invalid Animation");
		return;
	}

	if (!SkeletalMeshAsset)
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativeBodyAnimNotify. Reason: Invalid SkeletalMeshComponent");
		return;
	}

	if (!SkeletalMeshAsset->CloneMeshDescription(LODIndex, MeshDescription))
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativeBodyAnimNotify. Reason: Could not clone mesh description");
		return;
	}

	if (!PropsNotifyClass || PropsNotifyClass == URelativeBodyAnimNotifyBase::StaticClass())
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativePropsAnimNotify. Reason: Must specify valid relative body notify subclass to create");
		return;
	}

	if (!PhysicsAssetOverride)
	{
		UE_LOGF(LogAnimation, Warning, "RelativeBodyAnimModifier: No physics asset override specified, using skeletal mesh physics asset!")
	}

	//Cache
	UPhysicsAsset* PhysicsAsset = GetPhysicsAsset();
	FReferenceSkeleton& RefSkeleton = SkeletalMeshAsset->GetRefSkeleton();

	if (!PhysicsAsset)
	{
		UE_LOGF(LogAnimation, Error, "Cannot create RelativePropAnimNotify. Reason: Must specify valid physics asset");
		return;
	}

	// TODO: Move cached data out of modifier (or check if it's outdated and only rebuild in that case)
	CacheBodyDataForSourceMesh(CachedBodySourceData);

	const TArray<int32>& BodyIndicesParentBodyIndices = CachedBodySourceData.BodyIndicesParentBodyIndices;
	const TArray<bool>& BodyIndicesToIgnore = CachedBodySourceData.BodyIndicesToIgnore;
	const TArray<bool>& IsDomainBodyIndices = CachedBodySourceData.IsDomainBody;
	const TArray<int32>& BodyIndicesToBoneIndices = CachedBodySourceData.BodyIndicesToSourceBoneIndices;
	const TArray<int32>& BoneIndicesToBodyIndices = CachedBodySourceData.SourceBoneIndicesToBodyIndices;
	const TArray<TArray<int32>>& VertexIndicesInfluencedByBodyIndices = CachedBodySourceData.SourceVertexIndicesInfluencedByBodyIndices;

	const int32 NumBodies = BodyIndicesToBoneIndices.Num();
	const float HashGridCellSize = std::min(1.5f * ContactThreshold, 50.f); // the length of the cell size in the point hash gri

	// Setup and check for valid prop data
	PropAttachBones.Init(NAME_None, PropsData.Num());
	PropAttachTfms.Init(FTransform::Identity, PropsData.Num());
	bool PropsDataRequired = false;
	for (int32 PropsIdx = 0; PropsIdx < PropsData.Num(); PropsIdx++)
	{
		FMeshDescription* PropsMeshDescription = PropsData[PropsIdx].GetMeshDescription(0); // LOD 0
		if (!PropsMeshDescription)
		{
			UE_LOGF(LogAnimation, Warning, "RelativePropAnimModifier: Bad propdata at index [%d]. \n\tReason: PropsMeshDescription Error!", PropsIdx);
			continue;
		}
		
		if (PropsData[PropsIdx].PropStaticMeshAsset && PropsData[PropsIdx].PropAnimSequence && !PropsData[PropsIdx].PropSkeletalMeshAsset)
		{
			UE_LOGF(LogAnimation, Warning, "RelativePropAnimModifier: Bad propdata at index [%d]. \n\tReason: Need SkeletalMesh to support Prop Animation!", PropsIdx);
			continue;
		}
		
		FName CheckPropBone = GetPropAttachInfo(PropAttachTfms[PropsIdx], PropsData[PropsIdx], SkeletalMeshAsset);
		if (CheckPropBone == NAME_None)
		{
			UE_LOGF(LogAnimation, Warning, "RelativePropAnimModifier: Bad propdata at index [%d]. \n\tReason: Attach socket/bone '%ls' not found in skeletal mesh '%ls'!", PropsIdx, *PropsData[PropsIdx].SocketName.ToString(), *GetNameSafe(SkeletalMeshAsset));
			continue;
		}
		
		// NOTE: PropAttachBones[i] != NAME_None indicates valid prop data
		PropAttachBones[PropsIdx] = CheckPropBone;
		PropsDataRequired = true;
	}
	
	if (NotifyClass)
	{
		URelativeBodyAnimModifier::OnApply_Implementation(InAnimation);
	}
		
	if (!PropsDataRequired)
	{
		return;
	}
	
	// Process animation asset
	{
		const FAnimPoseEvaluationOptions AnimPoseEvalOptions{
			EAnimDataEvalType::Raw,
			true,
			false,
			false,
			nullptr,
			true,
			false
		};

		const float SequenceLength = InAnimation->GetPlayLength();
		const float SampleStep = 1.0f / FMath::Max(1.0f, static_cast<float>(SampleRate));
		const int SampleNum = FMath::TruncToInt(SequenceLength / SampleStep);
		
		URelativePropsBakeAnimNotify* BakedAnimNotifyInfo = nullptr;
		bool bUseDenseNotifyInfo = PropsNotifyClass->IsChildOf(URelativePropsBakeAnimNotify::StaticClass());
		TArray<FPropsMeshAndTree> PropsTreeData;
		PropsTreeData.SetNum(PropsData.Num());
		if (bUseDenseNotifyInfo)
		{
			FName TrackName("RelProps-<" + SkeletalMeshAsset->GetName() + ">");
			UE_LOGF(LogAnimation, Verbose, "TrackName %d: '%ls'.", GeneratedNotifyTracks.Num() - 1, *TrackName.ToString());
			const bool bDoesTrackNameAlreadyExist = UAnimationBlueprintLibrary::IsValidAnimNotifyTrackName(InAnimation, TrackName);
			if (!bDoesTrackNameAlreadyExist)
			{
				UAnimationBlueprintLibrary::AddAnimationNotifyTrack(InAnimation, TrackName, FLinearColor::MakeRandomColor());
				GeneratedNotifyTracks.Add(TrackName);
			}

			BakedAnimNotifyInfo = Cast<URelativePropsBakeAnimNotify>(UAnimationBlueprintLibrary::AddAnimationNotifyEvent(InAnimation, TrackName, 0., PropsNotifyClass));
			if (!BakedAnimNotifyInfo)
			{
				return;
			}
			
			for (int32 BodyIndex1 = 0; BodyIndex1 < NumBodies; ++BodyIndex1)
			{
				if (BodyIndicesToIgnore[BodyIndex1])
				{
					continue;
				}
				FName SourceBodyBoneName = RefSkeleton.GetBoneName(BodyIndicesToBoneIndices[BodyIndex1]);
				int32 SourceBodyIdx = PhysicsAsset->FindBodyIndex(SourceBodyBoneName);
				if (SourceBodyIdx == INDEX_NONE)
				{
					continue;
				}
				FKShapeElem* ShapeElem1 = PhysicsAsset->SkeletalBodySetups[SourceBodyIdx]->AggGeom.GetElement(0);
				FTransform3f BodyTransform1(ShapeElem1->GetTransform());
				FVector3f ElementScale3D1 = FRelativeBodyPhysicsUtils::CalcReferenceShapeScale3D(ShapeElem1);
				BodyTransform1.SetScale3D(ElementScale3D1);
				BakedAnimNotifyInfo->OffsetTransformsForBones.Emplace(SourceBodyBoneName, BodyTransform1);
			}
				
			for (size_t PropsIdx = 0; PropsIdx < PropsData.Num(); PropsIdx++)
			{
				if (PropAttachBones[PropsIdx] == NAME_None)
				{
					continue;
				}
				BakedAnimNotifyInfo->OffsetTransformsForBones.Emplace(PropAttachBones[PropsIdx],  PropAttachTfms[PropsIdx]);
				FMeshDescription* PropsMeshDescription = PropsData[PropsIdx].GetMeshDescription(0); // LOD 0
				PropsTreeData[PropsIdx].Init(PropsMeshDescription);
	
				for (int32 BodyIndex1 = 0; BodyIndex1 < NumBodies; ++BodyIndex1)
				{
					if (BodyIndicesToIgnore[BodyIndex1])
					{
						continue;
					}
					FName BoneName = RefSkeleton.GetBoneName(BodyIndicesToBoneIndices[BodyIndex1]);
					BakedAnimNotifyInfo->PropsPairs.Add(PropAttachBones[PropsIdx]);
					BakedAnimNotifyInfo->PropsPairs.Add(BoneName);
					BakedAnimNotifyInfo->bPropsPairsIsParentDominates.Add(IsDomainBodyIndices[BodyIndex1]);
				}
				if (bPropsFloorInfoBaking)
				{
					BakedAnimNotifyInfo->PropsPairs.Add(PropAttachBones[PropsIdx]);
					BakedAnimNotifyInfo->PropsPairs.Add(FName("Floor"));
					BakedAnimNotifyInfo->bPropsPairsIsParentDominates.Add(true);
				}
			}
			BakedAnimNotifyInfo->PropsPairsLocalReference.SetNumZeroed(BakedAnimNotifyInfo->PropsPairs.Num() * SampleNum);
			BakedAnimNotifyInfo->PropsPairsSampleTime.SetNumZeroed(SampleNum);
			BakedAnimNotifyInfo->NumSamples = SampleNum;
		}
		
		FScopedSlowTask BakePairVertsTask((float)SampleNum, LOCTEXT("BakeVertsTaskText", "Baking Props Pair Verts..."));
		BakePairVertsTask.MakeDialog();

		// Get ground levels and max speed values.
		TArray<FVector3f> VLocations;
		TArray<TArray<FVector3f>> PropsVLocations;
		for (int SampleIndex = 0; SampleIndex < SampleNum; ++SampleIndex)
		{
			BakePairVertsTask.EnterProgressFrame(1.0f);

			const float SampleTime = FMath::Clamp(static_cast<float>(SampleIndex) * SampleStep, 0.0f, SequenceLength);
			const float FutureSampleTime = FMath::Clamp((static_cast<float>(SampleIndex) + 1.0f) * SampleStep, 0.0f, SequenceLength);

			FAnimPose AnimPose;
			FAnimPose FutureAnimPose;

			UAnimPoseExtensions::GetAnimPoseAtTime(InAnimation, SampleTime, AnimPoseEvalOptions, AnimPose);
			UAnimPoseExtensions::GetAnimPoseAtTime(InAnimation, FutureSampleTime, AnimPoseEvalOptions, FutureAnimPose);

			TArray<FMatrix44f> CacheToLocals;
			GetRefToAnimPoseMatrices(CacheToLocals, AnimPose);
			GetSkinnedVertices(VLocations, CacheToLocals);
			if (BakedAnimNotifyInfo)
			{
				GetPropsVertices(PropsVLocations, AnimPose);
			}

			size_t Count = 0;
			TArray<FName> ContactedBodyPairs;
			TArray<FVector3f> ContactPoints;
			ContactPoints.Empty();
			if (!BakedAnimNotifyInfo)
			{
				continue;
			}
			else 
			{
				BakedAnimNotifyInfo->PropsPairsSampleTime[SampleIndex] = SampleTime;
				for (int32 PropsIdx = 0; PropsIdx < PropsData.Num(); PropsIdx++)
				{
					if (PropAttachBones[PropsIdx] == NAME_None)
					{
						continue;
					}
					
					FTransform AttachBoneTransform = UAnimPoseExtensions::GetBonePose(AnimPose, PropAttachBones[PropsIdx], EAnimPoseSpaces::World);
					FTransform PropsLocalToWorld = PropAttachTfms[PropsIdx] * AttachBoneTransform;
					
					if (PropsData[PropsIdx].PropSkeletalMeshAsset && PropsData[PropsIdx].PropAnimSequence)
					{
						float PropPlayhead = PropsData[PropsIdx].ComputePropAnimPlayhead(SampleTime);
						TArray<FMatrix44f> PropsCacheToLocals;
						PropsData[PropsIdx].GetRefToAnimPoseMatrices(PropPlayhead, PropsCacheToLocals);
						PropsData[PropsIdx].GetSkinnedVertices(PropsCacheToLocals, PropsVLocations[PropsIdx]);
						
						PropsTreeData[PropsIdx].UpdateVertexPositions(PropsVLocations[PropsIdx]);
						for (int32 i = 0; i < PropsVLocations[PropsIdx].Num(); ++i)
						{
							FVector LocalPos = PropAttachTfms[PropsIdx].TransformPosition(FVector(PropsVLocations[PropsIdx][i]));
							PropsVLocations[PropsIdx][i] = FVector3f(AttachBoneTransform.TransformPosition(LocalPos));
						}
					}
					
					for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
					{
						if (BodyIndicesToIgnore[BodyIndex])
						{
							continue;
						}

						size_t Offset = SampleIndex * BakedAnimNotifyInfo->PropsPairs.Num() + Count * 2;
						int32 NumVerts = VertexIndicesInfluencedByBodyIndices[BodyIndex].Num();

						double MinDist = DBL_MAX;
						FVector3d NearestPoint;
						int32 NearestQueryVertexIndex = INDEX_NONE;
						for (int32 QueryVertexIndex = 0; QueryVertexIndex < NumVerts; ++QueryVertexIndex) 
						{
							FVector3d VertexPosition(VLocations[VertexIndicesInfluencedByBodyIndices[BodyIndex][QueryVertexIndex]]);
							VertexPosition = PropsLocalToWorld.InverseTransformPosition(VertexPosition);
							FVector3d NearestPointTemp = PropsTreeData[PropsIdx].AABBTree.FindNearestPoint(VertexPosition);
							double DistSqr = FVector3d::DistSquared(VertexPosition, NearestPointTemp);
							// UE_LOGF(LogAnimation, Warning, "QueryVertexIndex %d: Dist=%f (%f,%f,%f).", QueryVertexIndex, DistSqr, NearestPointTemp.X, NearestPointTemp.Y, NearestPointTemp.Z);
							if (DistSqr < MinDist)
							{
								NearestQueryVertexIndex = QueryVertexIndex;
								NearestPoint = NearestPointTemp;
								MinDist = DistSqr;
							}
						}
						// UE_LOGF(LogAnimation, Warning, "MinDIst %f: (%f,%f,%f).", MinDist, NearestPoint.X, NearestPoint.Y, NearestPoint.Z);
						if (NearestQueryVertexIndex != INDEX_NONE)
						{
							FName BoneName1 = PropAttachBones[PropsIdx];
							FName BoneName2 = RefSkeleton.GetBoneName(BodyIndicesToBoneIndices[BodyIndex]);
							FVector3f ContactPoint1(PropsLocalToWorld.TransformPosition(NearestPoint));
							FVector3f ContactPoint2 = VLocations[VertexIndicesInfluencedByBodyIndices[BodyIndex][NearestQueryVertexIndex]];

							FVector3f Loc1(NearestPoint);
							FVector3f Loc2 = FRelativeBodyAnimUtils::CalcVertLocationInUnitBody(ContactPoint2, BoneName2, RefSkeleton, CacheToLocals, CachedBodySourceData.SourceRetargetGlobalPose, PhysicsAsset);
							FTransform3f PropOffsetTfm(BakedAnimNotifyInfo->OffsetTransformsForBones[BoneName1]);
							FTransform3f SourceBodyRot(FTransform(BakedAnimNotifyInfo->OffsetTransformsForBones[BoneName2].GetRotation()));

							// BakedAnimNotifyInfo->PropsPairsLocalReference[Offset + 0] = Loc1;
							// BakedAnimNotifyInfo->PropsPairsLocalReference[Offset + 1] = Loc2;
							BakedAnimNotifyInfo->PropsPairsLocalReference[Offset + 0] = PropOffsetTfm.TransformPosition(Loc1);
							BakedAnimNotifyInfo->PropsPairsLocalReference[Offset + 1] = SourceBodyRot.TransformPosition(Loc2);
							//UE_LOGF(LogAnimation, Warning, "NumVertsIDX %d: (%f,%f,%f).", NearestQueryVertexIndex, Loc1.X, Loc1.Y, Loc1.Z);
						}
						Count++;
					}
					if (bPropsFloorInfoBaking)
					{
						FName PropBone = PropAttachBones[PropsIdx];
						FTransform3f PropOffsetTfm(BakedAnimNotifyInfo->OffsetTransformsForBones[PropBone]);
						int32 Offset = SampleIndex * BakedAnimNotifyInfo->PropsPairs.Num() + Count * 2;
						int32 LowestVertexIndex = FindLowestIndex(PropsVLocations[PropsIdx]);
						int32 HighestVertexIndex=FindHighestIndex(PropsVLocations[PropsIdx]);

						BakedAnimNotifyInfo->PropsPairsLocalReference[Offset + 0] = FVector3f(AttachBoneTransform.InverseTransformPosition(FVector(PropsVLocations[PropsIdx][LowestVertexIndex])));
						BakedAnimNotifyInfo->PropsPairsLocalReference[Offset + 1] = FVector3f(AttachBoneTransform.InverseTransformPosition(FVector(PropsVLocations[PropsIdx][HighestVertexIndex])));

						Count++;
					}
				}
			}
		}
	}
}

FName URelativePropsAnimModifier::GetPropAttachInfo(FTransform& OutAttachTfm, const FPropsInfo& PropInfo, const USkeletalMesh* SkeletalMesh)
{
	FName AttachBoneName = NAME_None;
	OutAttachTfm = FTransform::Identity;

	FTransform BaseAttachTfm = FTransform::Identity;
	const USkeletalMeshSocket* Socket = SkeletalMesh->FindSocket(PropInfo.SocketName);
	if (Socket)
	{
		AttachBoneName = Socket->BoneName;
		OutAttachTfm = PropInfo.AttachTransform * FTransform(Socket->RelativeRotation, Socket->RelativeLocation, Socket->RelativeScale);
	}
	else if (SkeletalMesh->GetRefSkeleton().FindBoneIndex(PropInfo.SocketName) != INDEX_NONE)
	{
		AttachBoneName = PropInfo.SocketName;
		OutAttachTfm = PropInfo.AttachTransform;
	}
	
	return AttachBoneName;
}

void URelativePropsAnimModifier::GetPropsVertices(TArray<TArray<FVector3f>>& PropsVLocations, const FAnimPose& AnimPose)
{
	PropsVLocations.Init({}, PropsData.Num());
	for (int32 PropsIdx = 0; PropsIdx < PropsData.Num(); PropsIdx++)
	{
		if (PropAttachBones[PropsIdx] == NAME_None)
		{
			continue;
		}
		
		const FTransform AttachBoneTransform = UAnimPoseExtensions::GetBonePose(AnimPose, PropAttachBones[PropsIdx], EAnimPoseSpaces::World);
		
		// const FPositionVertexBuffer& PositionVertexBuffer = PropsData[PropsIdx].GetPositionVertexBuffer();
		// int32 NumVerts = PositionVertexBuffer.GetNumVertices();
		const FMeshDescription* MeshDescriptionPtr = PropsData[PropsIdx].GetMeshDescription(0);
		const int32 NumVerts = MeshDescriptionPtr->Vertices().Num();

		for (int32 i = 0; i < NumVerts; ++i)
		{
			// FVector LocalPos = PropAttachTfms[PropsIdx].TransformPosition(FVector(PositionVertexBuffer.VertexPosition(i)));
			FVector LocalPos = PropAttachTfms[PropsIdx].TransformPosition(FVector(MeshDescriptionPtr->GetVertexPosition(i)));
			PropsVLocations[PropsIdx].Add(FVector3f(AttachBoneTransform.TransformPosition(LocalPos)));
		}
	}

}

#undef LOCTEXT_NAMESPACE