// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationModifier.h"
#include "MeshDescription.h"
#include "UObject/ObjectPtr.h"

#include "RelativeBodyAnimModifier.generated.h"

class UPhysicsAsset;
class URelativeBodyAnimNotifyBase;
class USkeletalMesh;
struct FAnimPose;
struct FReferenceSkeleton;

USTRUCT(BlueprintType)
struct FRelativeBodySourceData
{
	GENERATED_BODY()

	/** The names of either controls of body modifiers (depending on context) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalIKRetargeter)
	TArray<int32> BodyIndicesParentBodyIndices;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalIKRetargeter)
	TArray<bool> BodyIndicesToIgnore;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalIKRetargeter)
	TArray<bool> IsDomainBody;

	/*Source Data*/
	TArray<FTransform> SourceRetargetGlobalPose;
	TArray<TArray<int32>> SourceVertexIndicesInfluencedByBodyIndices;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalIKRetargeter)
	TArray<FVector3f> SourceVLocations;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalIKRetargeter)
	TArray<int32> BodyIndicesToSourceBoneIndices;
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = PhysicalIKRetargeter)
	TArray<int32> SourceBoneIndicesToBodyIndices;
};

/**
 * Animation modifier for baking relative body relationships into anim notify
 */
UCLASS(BlueprintType, meta = (IsBlueprintBase = true))
class RELATIVEBODYANIMUTILS_API URelativeBodyAnimModifier : public UAnimationModifier
{
	GENERATED_BODY()

public:
	/** Begin UAnimationModifier interface */
	virtual void OnApply_Implementation(UAnimSequence* InAnimation) override;
	virtual void OnRevert_Implementation(UAnimSequence* InAnimation) override;
	/** End UAnimationModifier interface */
	
public:
	/** Rate used to sample the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (Units = "Hz", UIMin = 1))
	int SampleRate = 30;

	/** Threshold for determining if a bone pair can be considered to be having contact */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	float ContactThreshold = 150.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "SkeletalMeshAsset"))
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "PhysicsAssetOverride"))
	TObjectPtr<UPhysicsAsset> PhysicsAssetOverride;

	/** Bodies to be checked against contact bodies */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "DomainBodyNames"))
	TArray<FName> DomainBodyNames;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "ContactBodyNames"))
	TArray<FName> ContactBodyNames;

	// Relative Body Notify subclass to create
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "PhysicalIKRetargeterNotifyClass"))
	TSubclassOf<URelativeBodyAnimNotifyBase> NotifyClass = nullptr;

protected:
	FMeshDescription MeshDescription;
	FRelativeBodySourceData CachedBodySourceData;

	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Settings"))
	int32 LODIndex = 0;

	/** Keep track of to be generated tracks during modifier application */
	UPROPERTY(VisibleAnywhere, AdvancedDisplay, meta = (Category = "Settings"))
	TSet<FName> GeneratedNotifyTracks;

	UPhysicsAsset* GetPhysicsAsset() const;
	
	void CacheBodyDataForSourceMesh(FRelativeBodySourceData& OutSourceData);
	void GetRefToAnimPoseMatrices(TArray<FMatrix44f>& OutRefToPose, const FAnimPose& AnimPose) const;
	void GetSkinnedVertices(TArray<FVector3f>& VLocations, const TArray<FMatrix44f>& CacheToLocals);
};
