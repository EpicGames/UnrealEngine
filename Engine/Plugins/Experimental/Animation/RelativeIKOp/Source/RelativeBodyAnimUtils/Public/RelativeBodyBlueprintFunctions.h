// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RelativeBodyAnimModifier.h"
#include "RelativePropsAnimModifier.h"

#include "RelativeBodyBlueprintFunctions.generated.h"

struct FPropsInfo;

USTRUCT(BlueprintType)
struct RELATIVEBODYANIMUTILS_API FRelativeBodyAnimModifierOptions
{
	GENERATED_BODY()

	/** Rate used to sample the animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (Units = "Hz", UIMin = 1))
	int SampleRate = 30;

	/** Threshold for determining if a bone pair can be considered to be having contact */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	float ContactThreshold = 150.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "SkeletalMeshAsset"))
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "PhysicsAssetOverride"))
	TObjectPtr<UPhysicsAsset> PhysicsAssetOverride;

	/** Bodies to be checked against contact bodies */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "DomainBodyNames"))
	TArray<FName> DomainBodyNames;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "ContactBodyNames"))
	TArray<FName> ContactBodyNames;

	// Relative Body Notify subclass to create
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (TitleProperty = "PhysicalIKRetargeterNotifyClass"))
	TSubclassOf<URelativeBodyAnimNotifyBase> NotifyClass = nullptr;
};

USTRUCT(BlueprintType)
struct RELATIVEBODYANIMUTILS_API FPropBodyAnimModifierOptions : public FRelativeBodyAnimModifierOptions
{
	GENERATED_BODY()

	// Settings for each prop static mesh attachment to bake
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<FPropsInfo> PropMeshSettings;
	
	// Include floor constraint info in prop bake
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bPropsFloorInfoBaking = false;
	
	// Prop Notify subclass to create
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TSubclassOf<URelativePropsBakeAnimNotify> PropNotifyClass = nullptr;
};


/**
 * BPFL helper for setting up anim modifier 
 */
UCLASS(Transient, meta=(ScriptName="RelativeBodyAnimModifyHelper"))
class RELATIVEBODYANIMUTILS_API URelativeBodyAnimBlueprintFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	UFUNCTION(BlueprintCallable, Category = "Anim Modifier", meta=(DeprecatedFunction, DeprecationMessage="Set prop attach SocketName directly in Props Info"))
	static void SetPropAttachBone(UPARAM(ref) FPropsInfo& Info, FName AttachBoneName);
	
	UFUNCTION(BlueprintCallable, Category = "Anim Modifier", meta=(DeprecatedFunction, DeprecationMessage="Get prop attach SocketName directly from Props Info"))
	static FName GetPropAttachBone(const FPropsInfo& Info);
	
	UFUNCTION(BlueprintCallable, Category = "Anim Notify")
	static float GetSegmentStartTime(const FAnimNotifyEvent& AnimNotifyEvent);
	
	UFUNCTION(BlueprintCallable, Category = "Anim Modifier")
	static void SetRelativeAnimModifiers(UPARAM(ref) TArray<UAnimSequenceBase*>& AnimationList, const FPropBodyAnimModifierOptions& Options);
	
	UFUNCTION(BlueprintCallable, Category = "Anim Modifier")
	static void BulkUpdateRelativeBodyModifiers(UPARAM(ref) TArray<UAnimSequenceBase*>& AnimationList, const FRelativeBodyAnimModifierOptions& Options);
	
	UFUNCTION(BlueprintCallable, Category = "Anim Modifier")
	static void BulkUpdatePropBodyModifiers(UPARAM(ref) TArray<UAnimSequenceBase*>& AnimationList, const FPropBodyAnimModifierOptions& Options);
};
