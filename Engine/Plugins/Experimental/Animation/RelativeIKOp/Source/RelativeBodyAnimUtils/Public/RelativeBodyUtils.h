// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "UObject/ObjectPtr.h"
#include "UObject/NameTypes.h"

class UPhysicsAsset;
class USkeletalMesh;
struct FKShapeElem;
struct FReferenceSkeleton;

struct FRelativeBodyPhysicsUtils
{
	static FVector3f CalcReferenceShapeScale3D(FKShapeElem* ShapeElem);
	static FTransform GetBodyTransform(const UPhysicsAsset* PhysAsset, FName BoneName);
	static FKShapeElem* FindBodyShape(const UPhysicsAsset* PhysAsset, FName BoneName);
};

struct FRelativeBodyAnimUtils
{
	static FName FindParentBodyBoneName(const FName BoneName, const FReferenceSkeleton& RefSkeleton, const UPhysicsAsset* PhysicsAsset);
	static void GetRefGlobalPos(TArray<FTransform>& GlobalRefPose, const FReferenceSkeleton& RefSkeleton);

	static void GetRefPoseToLocalMatrices(TArray<FMatrix44f>& OutRefToLocals, TObjectPtr<USkeletalMesh> SkeletalMeshAsset, int32 LODIndex, const TArray<FTransform>& RetargetGlobalPose);
	static FVector3f CalcVertLocationInUnitBody(const FVector3f& RefPoint, const FName BodyBoneName, const FReferenceSkeleton& SourceRefSkeleton,  const TArray<FMatrix44f>& CacheToLocals, const TArray<FTransform>& SourceGlobalPose, UPhysicsAsset* SourcePhysicsAsset);
};
