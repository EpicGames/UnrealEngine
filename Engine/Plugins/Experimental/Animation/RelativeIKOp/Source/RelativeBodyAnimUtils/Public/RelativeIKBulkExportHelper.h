// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectMacros.h"

#include "RelativeIKBulkExportHelper.generated.h"

class UIKRetargeter;
class UAnimSequence;

/**
 * BPFL helper for running bulk anim retarget for export
 */
UCLASS(Transient, meta=(ScriptName="RelativeBodyBulkExportHelper"))
class RELATIVEBODYANIMUTILS_API URelativeIKBulkExportHelper : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = "Export Tools")
	static void BulkRetargetSequences(
		UIKRetargeter* RetargetAsset,
		UPARAM(ref) const TArray<UAnimSequence*>& RetargetSequenceList,
		const FString& OutPath,
		USkeletalMesh* SourceMesh = nullptr,
		USkeletalMesh* TargetMesh = nullptr);
};
