// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RelativeBodyAnimModifier.h"
#include "MeshDescription.h"
#include "UObject/ObjectPtr.h"

#include "RelativePropsAnimModifier.generated.h"

class UPhysicsAsset;
class URelativePropsBakeAnimNotify;
class USkeletalMesh;
struct FAnimPose;
struct FReferenceSkeleton;

class FPositionVertexBuffer;
struct FMeshDescription;

USTRUCT(BlueprintType)
struct FPropsInfo
{
	GENERATED_BODY()
	// Prop mesh to preview
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	TObjectPtr<UStaticMesh> PropStaticMeshAsset;
	
	// Prop mesh to preview
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	TObjectPtr<USkeletalMesh> PropSkeletalMeshAsset;

	// Source skeletal mesh bone to attach prop to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	FName SocketName;

	// Local transform of prop relative to attach bone
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	FTransform AttachTransform;
	
	// Prop mesh to anim sequence
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	TObjectPtr<UAnimSequence> PropAnimSequence;
	
	// Prop mesh to anim sequence start time
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	float PropAnimStartTime = 0.0f;
	
	// Prop mesh to anim sequence is looping
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop", meta=(ReinitializeOnEdit))
	bool PropAnimIsLooping = true;

public:
	const FPositionVertexBuffer& GetPositionVertexBuffer();
	FMeshDescription* GetMeshDescription(int32 LodIndex);
	
	float ComputePropAnimPlayhead(float CharacterPlayhead) const;
	void GetSkinnedVertices(const TArray<FMatrix44f>& RefToPoseMatrices, TArray<FVector3f>& VLocation);
	void GetRefToAnimPoseMatrices(float AnimPlayhead, TArray<FMatrix44f>& OutRefToPose) const;
};

/**
 * Animation modifier for baking relative body relationships into anim notify
 */
UCLASS(meta = (IsBlueprintBase = true))
class RELATIVEBODYANIMUTILS_API URelativePropsAnimModifier : public URelativeBodyAnimModifier
{
	GENERATED_BODY()

public:
	/** Begin UAnimationModifier interface */
	virtual void OnApply_Implementation(UAnimSequence* InAnimation) override;
	/** End UAnimationModifier interface */

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Settings", meta = (TitleProperty = "PropsData"))
	TArray<FPropsInfo> PropsData;

	/**  Relative Props Notify subclass to create */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (TitleProperty = "PropsNotifyClass"))
	TSubclassOf<URelativePropsBakeAnimNotify> PropsNotifyClass = nullptr;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (TitleProperty = "PropsFloorInfoBaking"))
	bool bPropsFloorInfoBaking = false;
	
private:
	void GetPropsVertices(TArray<TArray<FVector3f>>& PropsVLocations, const FAnimPose& AnimPose);
	
	static FName GetPropAttachInfo(FTransform& OutAttachTfm, const FPropsInfo& PropInfo, const USkeletalMesh* SkeletalMesh);
	
	// Actual bone names (pulled from socket info or prop info)
	TArray<FName> PropAttachBones;
	
	// Baked down socket/prop attachments
	TArray<FTransform> PropAttachTfms;
};