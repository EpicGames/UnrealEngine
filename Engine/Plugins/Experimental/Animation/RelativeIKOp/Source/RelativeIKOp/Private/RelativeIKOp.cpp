// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelativeIKOp.h"

#include "IKRigDebugRendering.h"
#include "RelativeBodyAnimNotifies.h"
#include "PhysicsBodyHelpers.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Components/SkeletalMeshComponent.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"
#include "Eigen/SVD"

#include "AnimationRuntime.h"
#include "BoneContainer.h"
#include "BoneIndices.h"
#include "BonePose.h"

#if WITH_EDITOR
#include "PrimitiveDrawingUtils.h"
#include "Engine/EngineTypes.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RelativeIKOp)

#define LOCTEXT_NAMESPACE "RelativeIKOp"

template<int P>
double RationalSmoothStepdown(double Distance, double DistThreshold, double DistHalfPointLambda)
{
	constexpr double kEps = UE_DOUBLE_SMALL_NUMBER;
	
	double x = FMath::Clamp(Distance / DistThreshold, 0., 1.);
	double a = FMath::Clamp(DistHalfPointLambda, kEps, 1.-kEps);
	const double ax = (1.0-a)*x;
	const double iax = a*(1.0-x);
	
	double r1 = ax;
	double r2 = iax;
	if constexpr (P > 1) 
	{
		for (int i = 0; i < P - 1; ++i)
		{
			r1 *= ax;
			r2 *= iax;
		}
	}
	return (r2) / (r1 + r2);
}

template double RationalSmoothStepdown<1>(double, double, double);
template double RationalSmoothStepdown<2>(double, double, double);
template double RationalSmoothStepdown<3>(double, double, double);
template double RationalSmoothStepdown<4>(double, double, double);
template double RationalSmoothStepdown<5>(double, double, double);

const UClass* FIKRetargetRelativeIKOpSettings::GetControllerType() const
{
	return UIKRetargetRelativeIKController::StaticClass();
}

void FIKRetargetRelativeIKOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	static TArray<FName> PropertiesToIgnore = {};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetRelativeIKOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

bool FIKRetargetRelativeIKOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;

	// this op requires a parent to supply an IK Rig
	if (!ensure(InParentOp))
	{
		return false;
	}

	// validate that an IK rig has been assigned
	const FIKRetargetRunIKRigOp* ParentRigOp = reinterpret_cast<const FIKRetargetRunIKRigOp*>(InParentOp);
	if (!ParentRigOp || ParentRigOp->Settings.IKRigAsset == nullptr)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingIKRig", "{0}, is missing an IK rig. No goals can be updated. "), FText::FromName(GetName())));
		return false;
	}
	
	TargetRelativeIKPhysicsAsset = Settings.TargetPhysicsAssetOverride;
	TargetIntersectOpPhysicsAsset= Settings.TargetIntersectOpPhysicsAssetOverride;
	if (!TargetRelativeIKPhysicsAsset)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingTargetPhysicsAssets", "{0}: Target RelativeIK Physics Assets must be specified. "), FText::FromName(GetName())));
		return false;
	}
	if (Settings.bEnableIntersectPushOut && !TargetIntersectOpPhysicsAsset)
	{
		TargetIntersectOpPhysicsAsset = TargetRelativeIKPhysicsAsset;
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingTargetIntersectOpPhysicsAssets", "{0}: Missing TargetIntersectOpPhysicsAsset Override, use TargetRelativeIKPhysicsAsset Override instead!"), FText::FromName(GetName())));
	}

	UpdateCacheBoneChains(InProcessor, ParentRigOp);
	UpdateCacheSkelInfo(InSourceSkeleton, InTargetSkeleton);
	
	// Add all pin bones to the source/target skel lookup tables
	InitializePinData(InProcessor);
	
	InitializeIntersectionData(InProcessor);

	// Force update cache info if new anim montage found
	MontageInstance = nullptr;
	CacheSourceAnimSequence = nullptr;
	
	bIsInitialized = true;
	return bIsInitialized;
}

void FIKRetargetRelativeIKOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
#if WITH_EDITOR
	ResetDebugInfo();
#endif //WITH_EDITOR
	
	if (InProcessor.IsIKForcedOff())
	{
		return;
	}
	
	if (!bIsInitialized || !CacheSourceAnimSequence || AnimSeqPlayHead < 0.0f)
	{
		return;
	}
	
	// TODO: Figure out how to move these out of processing functions?
	FDebugRelativeIKDrawInfo LocalDebugInfo;
	TArray<FDebugRelativeTargetPairSpace> LocalTargetPairSpaces;

	int32 NumMeshPairs = (CachedNotifyInfo) ? CachedNotifyInfo->bBodyPairsIsParentDominates.Num() : 0;
	int32 NumPropPairs = (CachedPropNotifyInfo) ? CachedPropNotifyInfo->bPropsPairsIsParentDominates.Num() : 0;
	int32 TotalPairs = NumMeshPairs + NumPropPairs;
	
	int32 NumMeshBodies = (CachedNotifyInfo) ? CachedNotifyInfo->BodyPairs.Num() : 0;
	int32 NumPropBones = (CachedPropNotifyInfo) ? CachedPropNotifyInfo->PropsPairs.Num() : 0;
	int32 TotalBodies = NumMeshBodies + NumPropBones;

	if ( TotalBodies == 0 )
	{
		return;
	}
	
#if WITH_EDITOR
	LocalDebugInfo.SourcePairVerts.Reserve(TotalPairs);
	LocalDebugInfo.TargetPairVerts.Reserve(TotalPairs);
	
	LocalTargetPairSpaces.SetNumZeroed(TotalBodies);
#endif //WITH_EDITOR
	
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	TArray<FIKRigGoal>& IKGoals = GoalContainer.GetGoalArray();

	// Copy/cache (pin) prop bones in target using pin bone settings
	CachePinBoneTransforms(InProcessor, InSourceGlobalPose, OutTargetGlobalPose);
	
	// Get Array of bones we may amplify weight on
	TArray<FName> AmplifyBones;
	if (Settings.bApplyImportanceWeighting)
	{
		AmplifyBones.Reset(IKGoals.Num() + Settings.PinPropBones.Num());
		for (const FIKRigGoal& Goal : IKGoals)
		{
			AmplifyBones.Add(Goal.BoneName);
		}
	}

	// TODO: Should we just use initial chain lengths?
	CacheChainLengthMap.Reset();
	// Remove previous frame cache chain start data
	CacheChainStartMap.Reset();

	const double SourceScale = InProcessor.GetSourceScaleFactor();
	if (CachedNotifyInfo)
	{
		int32 NumMeshSamples = CachedNotifyInfo->NumSamples;
		TConstArrayView<FVector3f> FramePointView = ApplyTemporalSmoothing(AnimSeqPlayHead, MeshSampleRate, NumMeshSamples, NumMeshBodies, CachedNotifyInfo->BodyPairsLocalReference);
		ComputeFramePairSpaces(SourceScale, InSourceGlobalPose, OutTargetGlobalPose, CacheMeshPairOffset, FramePointView, LocalDebugInfo, LocalTargetPairSpaces);
	}

	if (CachedPropNotifyInfo)
	{
		int32 NumPropSamples = CachedPropNotifyInfo->NumSamples;
		TConstArrayView<FVector3f> FramePropView = ApplyTemporalSmoothing(AnimSeqPlayHead, PropSampleRate, NumPropSamples, NumPropBones, CachedPropNotifyInfo->PropsPairsLocalReference);
		ComputeFramePropSpaces(SourceScale, AmplifyBones, InSourceGlobalPose, OutTargetGlobalPose, CachePropPairOffset, FramePropView, LocalDebugInfo, LocalTargetPairSpaces);
	}
	
	if (Settings.bApplyImportanceWeighting)
	{
		AmplifyImportanceWeights();
	}
	
	LocalDebugInfo.TargetGoals.Reserve(CacheSourceBodyEffectVertIdx.Num());
	
	// Update bone goals to relative IK positions
	if (Settings.bMultiBoneSolve)
	{
		if (Settings.bSolveWithPropScale)
		{
			ApplyMatrixBodyPairTargetsWithPropScale(IKGoals, OutTargetGlobalPose, LocalDebugInfo, LocalTargetPairSpaces);
		}
		else
		{
			ApplyMatrixBodyPairTargets(IKGoals, OutTargetGlobalPose, LocalDebugInfo, LocalTargetPairSpaces);
		}
	}
	else
	{
		ApplyAverageBodyPairTargets(IKGoals, OutTargetGlobalPose, LocalDebugInfo, LocalTargetPairSpaces);
	}

	if (Settings.bEnableIntersectPushOut)
	{
		ApplyIntersectPushOut(IKGoals, OutTargetGlobalPose, LocalDebugInfo);
	}
#if WITH_EDITOR
	LocalDebugInfo.TargetDomainBones.Reset(CacheSourceEffectBones.Num());
	for (int PairIdx=0; PairIdx<NumMeshPairs; PairIdx++)
	{
		if(CachedNotifyInfo->bBodyPairsIsParentDominates[PairIdx])
		{
			FName DomainBody = ApplyBodyMap(CachedNotifyInfo->BodyPairs[2*PairIdx]);
			if (!LocalDebugInfo.TargetDomainBones.Contains(DomainBody))
			{
				LocalDebugInfo.TargetDomainBones.Add(DomainBody);
			}
		}
	}
	for (int PairIdx=0; PairIdx<NumPropPairs; PairIdx++)
	{
		if(CachedPropNotifyInfo->bPropsPairsIsParentDominates[PairIdx])
		{
			FName DomainBody = ApplyBodyMap(CachedPropNotifyInfo->PropsPairs[2*PairIdx+1]);
			if (!LocalDebugInfo.TargetDomainBones.Contains(DomainBody))
			{
				LocalDebugInfo.TargetDomainBones.Add(DomainBody);
			}
		}
	}
	
	UpdateDebugInfo(SourceScale, InSourceGlobalPose, OutTargetGlobalPose, LocalDebugInfo);
#endif
}

void FIKRetargetRelativeIKOp::RunAfterParent(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	if (!Settings.bApplyPostSolvePinning)
	{
		return;
	}
	
	for (const FRelativeIKPinBoneData& PinData : CachePinBoneData)
	{
		FName SourcePinBone = PinData.SourceBone;
		FName TargetPinBone = PinData.TargetBone;
		if (!CacheSourceSkelIndices.Contains(SourcePinBone) || !CacheTargetSkelIndices.Contains(TargetPinBone))
		{
			continue;
		}
		
		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetPinBone];
		int32 CachePropIdx = CachePinSourcePropBones.Find(SourcePinBone);
		if (CachePinTargetPropTransforms.IsValidIndex(CachePropIdx))
		{
			OutTargetGlobalPose[TargetBoneIdx] = CachePinTargetPropTransforms[CachePropIdx];
		}
		else
		{
			OutTargetGlobalPose[TargetBoneIdx] = ComputePinBoneTransform(PinData, InProcessor, InSourceGlobalPose, OutTargetGlobalPose);
			OutTargetGlobalPose[TargetBoneIdx].SetScale3D(FVector(PinData.PropScalar));
		}
	}
}

void FIKRetargetRelativeIKOp::ComputeFramePairSpaces(
	double SourceScale,
	const TArray<FTransform>& SourcePose,
	const TArray<FTransform>& TargetPose,
	int32 PairOffset,
	TConstArrayView<FVector3f> FramePointView,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	int32 PairCount = CachedNotifyInfo->bBodyPairsIsParentDominates.Num();
	const double InvSourceScale = (Settings.bIgnoreSourceScale) ? 1.0 / SourceScale : 1.0;
	
	const double RetargetContactAlpha = FMath::Clamp(Settings.RetargetContactAlpha, 0.0, 1.0);
	const double RetargetSpringAlpha = FMath::Clamp(Settings.RetargetSpringAlpha, 0.0, 1.0);
	// Get pair target body verts and weights
	for (int PairIdx = 0; PairIdx < PairCount; ++PairIdx)
	{
		// TODO: Allow selecting a subset of body pairs in retargeter
		const int32 OutPairIdx = PairIdx + PairOffset;
		FrameBoneConstraints[2*OutPairIdx].Reset();
		FrameBoneConstraints[2*OutPairIdx+1].Reset();
		
		FName SourcePrimaryBone = CachedNotifyInfo->BodyPairs[2*PairIdx];
		FName SourceOtherBone = CachedNotifyInfo->BodyPairs[2*PairIdx + 1];

		FName TargetPrimaryBone = ApplyBodyMap(SourcePrimaryBone);
		FName TargetOtherBone = ApplyBodyMap(SourceOtherBone);

		if (TargetPrimaryBone == NAME_None || TargetOtherBone == NAME_None)
		{
			continue;
		}

		if (!CacheSourceSkelIndices.Contains(SourcePrimaryBone) || !CacheSourceSkelIndices.Contains(SourceOtherBone))
		{
			continue;
		}

		if (!CacheTargetSkelIndices.Contains(TargetPrimaryBone) || !CacheTargetSkelIndices.Contains(TargetOtherBone))
		{
			continue;
		}

		if (!HasBoneDelta(TargetPrimaryBone) || !HasBoneDelta(TargetOtherBone))
		{
			continue;
		}

		bool bPrimaryMovable = CachedNotifyInfo->bBodyPairsIsParentDominates[PairIdx];

		int32 SourcePrimaryPoseIdx = CacheSourceSkelIndices[SourcePrimaryBone];
		int32 SourceOtherPoseIdx = CacheSourceSkelIndices[SourceOtherBone];
		
		// Compute source transforms and source-space vertex distance
		// const FVector3f& LocalPrimaryVert = CachedNotifyInfo->BodyPairsLocalReference[FramePairIdx + 2*PairIdx];
		// const FVector3f& LocalOtherVert = CachedNotifyInfo->BodyPairsLocalReference[FramePairIdx + 2*PairIdx + 1];
		FVector SourceBoneBakePrimaryV(FramePointView[2*PairIdx]);
		FVector SourceBoneBakeOtherV(FramePointView[2*PairIdx+1]);

		FBodyTransform SourcePrimaryTfm;
		FBodyTransform SourceOtherTfm;
		ComputeSourceBodyTransform(SourcePrimaryTfm, SourcePrimaryBone, SourcePose[SourcePrimaryPoseIdx], SourceScale);
		ComputeSourceBodyTransform(SourceOtherTfm, SourceOtherBone, SourcePose[SourceOtherPoseIdx], SourceScale);

		FVector SourcePrimaryCompV = ApplyBodyTransform(SourcePrimaryTfm, SourceBoneBakePrimaryV);
		FVector SourceOtherCompV = ApplyBodyTransform(SourceOtherTfm, SourceBoneBakeOtherV);

		// Rescale components for distance checks if "Ignore sources scale" is on
		// NOTE: This works because we don't need absolute coordinates (just distance).
		//       If absolute coords needed then must properly account for scale pivot
		FVector RescaleSourcePrimaryV = InvSourceScale * SourcePrimaryCompV;
		FVector RescaleSourceOtherV = InvSourceScale * SourceOtherCompV;
		
		double SourceVDist = FVector::Distance(RescaleSourcePrimaryV, RescaleSourceOtherV);
		double LinearWeight = FMath::Clamp(1.0 - (SourceVDist / Settings.DistanceThreshold), 0.0, 1.0);
		double Weight = GetDistanceWeight(Settings.DistanceWeightMode, SourceVDist, Settings.DistanceThreshold, Settings.DistHalfPointLambda);

		// Compute Source pair verts as represented in alternate pair body primary(_P) or other(_O) local bake space
		FVector SourcePrimaryV_O = InverseBodyTransform(SourceOtherTfm, SourcePrimaryCompV);
		FVector SourceOtherV_P = InverseBodyTransform(SourcePrimaryTfm, SourceOtherCompV);
		
		//---------------------------------
		// Get verts retargeted using physics body oriented scale
		int32 TargetPrimaryPoseIdx = CacheTargetSkelIndices[TargetPrimaryBone];
		int32 TargetOtherPoseIdx = CacheTargetSkelIndices[TargetOtherBone];

		FBodyTransform TargetPrimaryTfm;
		FBodyTransform TargetOtherTfm;
		ComputeTargetBodyTransform(TargetPrimaryTfm, TargetPrimaryBone, TargetPose[TargetPrimaryPoseIdx]);
		ComputeTargetBodyTransform(TargetOtherTfm, TargetOtherBone, TargetPose[TargetOtherPoseIdx]);

		// Compute "primary" body vert represented in component space relative to primary (_P) and other (_O) bodies
		FVector TargetPrimaryCompV_P = ApplyBodyTransform(TargetPrimaryTfm, SourceBoneBakePrimaryV);
		FVector TargetPrimaryCompV_O = ApplyBodyTransform(TargetOtherTfm, SourcePrimaryV_O);

		// Compute "other" body vert represented in component space relative to primary (_P) and other (_O) bodies
		FVector TargetOtherCompV_O = ApplyBodyTransform(TargetOtherTfm, SourceBoneBakeOtherV);
		FVector TargetOtherCompV_P = ApplyBodyTransform(TargetPrimaryTfm, SourceOtherV_P);
		
		// These are the target midpoint positions used for spring constraints in PhysicalIK
		FVector TargetSpring_P = FMath::Lerp(TargetPrimaryCompV_P,TargetOtherCompV_P, RetargetSpringAlpha);
		FVector TargetSpring_O = FMath::Lerp(TargetPrimaryCompV_O, TargetOtherCompV_O, RetargetSpringAlpha);

		// Offset vector (Other -> Primary) to Lerp for dist alpha
		FVector TargetOffsetVec = TargetSpring_P - TargetSpring_O;

		FVector TargetPrimaryBoneLoc = TargetPose[TargetPrimaryPoseIdx].GetLocation();
		FVector TargetOtherBoneLoc = TargetPose[TargetOtherPoseIdx].GetLocation();
	
		FDoubleInterval FeasibleRange_O(0.0, 1.0);
		FDoubleInterval FeasibleRange_P(0.0, 1.0);
		UpdateCacheChainInfo(TargetPose, TargetOtherBone);
		UpdateCacheChainInfo(TargetPose, TargetPrimaryBone);
		ComputeFeasibilityRange(FeasibleRange_O, TargetOtherBone, TargetOtherBoneLoc, TargetOffsetVec);
		if (bPrimaryMovable)
		{
			ComputeFeasibilityRange(FeasibleRange_P, TargetPrimaryBone, TargetPrimaryBoneLoc, -TargetOffsetVec);
		}

		double ContactBlendWeight = (Settings.bBidirectionalDistanceContactBlend) ? LinearWeight : 0.5;
		double FeasibilityWeight_O = FMath::Lerp(FeasibleRange_O.Min, FeasibleRange_O.Max, ContactBlendWeight);
		double FeasibilityWeight_P = FMath::Lerp(FeasibleRange_P.Min, FeasibleRange_P.Max, ContactBlendWeight);
		FVector WeightedTargetOffset_O = TargetOffsetVec * (Settings.bMultiBoneSolve? 0.5*(FeasibilityWeight_O+FeasibilityWeight_P): FeasibilityWeight_O);
		FVector WeightedTargetOffset_P = -TargetOffsetVec * (Settings.bMultiBoneSolve? 0.5*(FeasibilityWeight_O+FeasibilityWeight_P): FeasibilityWeight_P);
		
		FrameBoneConstraints[2*OutPairIdx].ContactOffset = TargetSpring_P-TargetPrimaryBoneLoc;
		FrameBoneConstraints[2*OutPairIdx+1].ContactOffset = TargetSpring_O-TargetOtherBoneLoc;
		FrameBoneConstraints[2*OutPairIdx].Offset = WeightedTargetOffset_P;
		FrameBoneConstraints[2*OutPairIdx+1].Offset = WeightedTargetOffset_O;
		FrameBoneConstraints[2*OutPairIdx].FeasibilityWeight = FeasibilityWeight_P;
		FrameBoneConstraints[2*OutPairIdx+1].FeasibilityWeight = FeasibilityWeight_O;
		FrameBoneConstraints[2*OutPairIdx].Weight = Weight;
		FrameBoneConstraints[2*OutPairIdx+1].Weight = Weight;
		FrameBoneConstraints[2*OutPairIdx].SourceDist = SourceVDist;
		FrameBoneConstraints[2*OutPairIdx+1].SourceDist = SourceVDist;
		
#if WITH_EDITOR
		// Debug draw verts
		if (Settings.bDebugDraw)
		{
			DebugInfo.SourcePairVerts.Add({SourcePrimaryCompV, SourceOtherCompV, false, FVector::ZeroVector, FVector::ZeroVector, Weight});
			if (Settings.DebugFullRetargetPairBones.Contains(TargetPrimaryBone))
			{
				DebugInfo.TargetPairVerts.Add({TargetPrimaryCompV_P, TargetOtherCompV_O, true,TargetOtherCompV_P, TargetPrimaryCompV_O, Weight});
			}
			else
			{
				DebugInfo.TargetPairVerts.Add({TargetPrimaryCompV_P, TargetOtherCompV_O, false, FVector::ZeroVector, FVector::ZeroVector, Weight});
			}
			
			DebugPairSpaces[2*OutPairIdx] = FDebugRelativeTargetPairSpace{TargetPrimaryCompV_P, TargetPrimaryCompV_P-TargetOffsetVec, FeasibilityWeight_P, FeasibleRange_P};
			DebugPairSpaces[2*OutPairIdx+1] = FDebugRelativeTargetPairSpace{TargetOtherCompV_O, TargetOtherCompV_O+TargetOffsetVec, FeasibilityWeight_O, FeasibleRange_O};
		}
#endif //WITH_EDITOR
	}
}

void FIKRetargetRelativeIKOp::ComputeFramePropSpaces(
	double SourceScale,
	const TArray<FName>& AmplifyBones,
	const TArray<FTransform>& SourcePose,
	const TArray<FTransform>& TargetPose,
	int32 PairOffset,
	TConstArrayView<FVector3f> FramePointView,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	int32 PairCount = CachedPropNotifyInfo->bPropsPairsIsParentDominates.Num();
	if (PairCount <= 0)
	{
		return;
	}

	FTransform SourceScaleTfm(FQuat::Identity, FVector::ZeroVector, FVector(SourceScale));
	const double InvSourceScale = (Settings.bIgnoreSourceScale) ? 1.0 / SourceScale : 1.0;
	
	const double RetargetContactAlpha = FMath::Clamp(Settings.RetargetContactAlpha, 0.0, 1.0);
	const double RetargetSpringAlpha = FMath::Clamp(Settings.RetargetSpringAlpha, 0.0, 1.0);
	// Get pair target body verts and weights
	for (int PairIdx = 0; PairIdx < PairCount; ++PairIdx)
	{
		const int32 OutPairIdx = PairIdx + PairOffset;
		FrameBoneConstraints[2*OutPairIdx].Reset();
		FrameBoneConstraints[2*OutPairIdx+1].Reset();
		
		FName SourcePropBone = CachedPropNotifyInfo->PropsPairs[2 * PairIdx];
		FName TargetPropBone = ApplyBodyMap(SourcePropBone);
		FName SourceBodyBone = CachedPropNotifyInfo->PropsPairs[2*PairIdx + 1];
		FName TargetBodyBone = ApplyBodyMap(SourceBodyBone);

		if (TargetPropBone == NAME_None || !CacheSourceSkelIndices.Contains(SourcePropBone) || !CacheTargetSkelIndices.Contains(TargetPropBone) )
		{
			continue;
		}

		if (SourceBodyBone==FName("Floor"))
		{
			int32 PinPropIdx = CachePinSourcePropBones.Find(SourcePropBone);
			if (PinPropIdx == INDEX_NONE)
			{
				continue;
			}
			
			int32 SourcePropPoseIdx = CacheSourceSkelIndices[SourcePropBone];
			int32 TargetPropPoseIdx = CacheTargetSkelIndices[TargetPropBone];
			
			FTransform SourcePropTfm = SourceScaleTfm * SourcePose[SourcePropPoseIdx];
			FTransform TargetPropBoneDelta = GetTargetBoneDelta(TargetPropBone);
			FTransform TargetPropTfm = TargetPropBoneDelta * TargetPose[TargetPropPoseIdx];
			
			FVector SourcePropBakeZMin(FramePointView[2*PairIdx]);
			FVector SourcePropBakeZMax(FramePointView[2*PairIdx+1]);

			FVector SourcePropCompZMin = SourcePropTfm.TransformPosition(SourcePropBakeZMin);
			FVector SourcePropCompZMax = SourcePropTfm.TransformPosition(SourcePropBakeZMax);

			FVector TargetPropCompZMin = TargetPropTfm.TransformPosition(SourcePropBakeZMin);
			FVector TargetPropCompZMax = TargetPropTfm.TransformPosition(SourcePropBakeZMax);
			
			// Rescale components for distance checks if "Ignore sources scale" is on
			// NOTE: This works because we don't need absolute coordinates (just distance).
			//       If absolute coords needed then must properly account for scale pivot
			FVector RescaleSourcePropZMin = InvSourceScale * SourcePropCompZMin;
			FVector RescaleSourcePropZMax = InvSourceScale * SourcePropCompZMax;
			double SourceZDist = RescaleSourcePropZMin.Z;
			double SourceWeight = GetDistanceWeight(Settings.DistanceWeightMode, SourceZDist, Settings.FloorThreshold, Settings.DistHalfPointLambda);
			double TargetZDist = TargetPropCompZMin.Z;
			double TargetWeight = GetDistanceWeight(Settings.DistanceWeightMode, TargetZDist, Settings.FloorThreshold, Settings.DistHalfPointLambda);
			
			// Offset vector (Prop -> Floor) to Lerp for dist alpha
			double FloorOffsetZMin = Settings.DistHalfPointLambda*Settings.FloorThreshold-TargetPropCompZMin.Z;
			FVector TargetOffsetZMin = SourcePropCompZMin - TargetPropCompZMin;
			FVector TargetOffsetZMax = SourcePropCompZMax - TargetPropCompZMax;
			double Weight = Settings.FloorConstraintScalar*(SourceWeight*SourceWeight + (1-SourceWeight)*TargetWeight);

			FrameBoneConstraints[2*OutPairIdx].ConstraintType = ERelativeIKFrameBoneConstraintType::Z;
			FrameBoneConstraints[2*OutPairIdx+1].ConstraintType = ERelativeIKFrameBoneConstraintType::Z;
			FrameBoneConstraints[2*OutPairIdx].ContactOffset = {0,0,(TargetPropCompZMin.Z - TargetPose[TargetPropPoseIdx].GetLocation().Z)};
			FrameBoneConstraints[2*OutPairIdx+1].ContactOffset = {0,0,(TargetPropCompZMax.Z - TargetPose[TargetPropPoseIdx].GetLocation().Z)};
			FrameBoneConstraints[2*OutPairIdx].Offset = {0,0,SourceWeight*TargetOffsetZMin.Z + (1-SourceWeight)*FloorOffsetZMin};
			FrameBoneConstraints[2*OutPairIdx+1].Offset = {0,0,SourceWeight*TargetOffsetZMax.Z + (1-SourceWeight)*FloorOffsetZMin};
			FrameBoneConstraints[2*OutPairIdx].Weight = Weight;
			FrameBoneConstraints[2*OutPairIdx+1].Weight = Weight;
			FrameBoneConstraints[2*OutPairIdx].SourceDist = SourceZDist;
			FrameBoneConstraints[2*OutPairIdx+1].SourceDist = SourceZDist;

#if WITH_EDITOR
			// Debug draw verts
			if (Settings.bDebugDraw)
			{
				FVector SourceFloorZMin = SourcePropCompZMin;
				SourceFloorZMin.Z = 0;
				FVector TargetFloorZMin = TargetPropCompZMin;
				TargetFloorZMin.Z = 0;
				
				DebugInfo.SourcePairVerts.Add({SourcePropCompZMin, SourceFloorZMin, false, FVector::ZeroVector, FVector::ZeroVector, Weight});
				DebugInfo.TargetPairVerts.Add({TargetPropCompZMin, TargetFloorZMin, false, FVector::ZeroVector, FVector::ZeroVector, Weight});
			}
#endif //WITH_EDITOR
			continue;
		}
		
		if (TargetBodyBone == NAME_None || !CacheSourceSkelIndices.Contains(SourceBodyBone) || !CacheTargetSkelIndices.Contains(TargetBodyBone))
		{
			continue;
		}

		if (!HasBoneDelta(TargetPropBone) || !HasBoneDelta(TargetBodyBone))
		{
			continue;
		}
		
		// Potentially amplify weight of this pair by total relative weight
		bool bAmplify = AmplifyBones.Contains(TargetBodyBone);
		
		int32 SourcePropPoseIdx = CacheSourceSkelIndices[SourcePropBone];
		int32 SourceBodyPoseIdx = CacheSourceSkelIndices[SourceBodyBone];
		
		// Compute source transforms and source-space vertex distance
		// const FVector3f& LocalPropVert = FramePointView[2*PairIdx];
		// const FVector3f& LocalBodyVert = FramePointView[2*PairIdx+1];
		FVector SourceBoneBakePropV(FramePointView[2*PairIdx]);
		FVector SourceBoneBakeBodyV(FramePointView[2*PairIdx+1]);

		bool bBodyMovable = CachedPropNotifyInfo->bPropsPairsIsParentDominates[PairIdx];
		
		FTransform SourcePropTfm = SourceScaleTfm * SourcePose[SourcePropPoseIdx];
		FBodyTransform SourceBodyTfm;
		ComputeSourceBodyTransform(SourceBodyTfm, SourceBodyBone, SourcePose[SourceBodyPoseIdx], SourceScale);
		
		FVector SourcePropCompV = SourcePropTfm.TransformPosition(SourceBoneBakePropV);
		FVector SourceBodyCompV = ApplyBodyTransform(SourceBodyTfm, SourceBoneBakeBodyV);

		// Rescale components for distance checks if "Ignore sources scale" is on
		// NOTE: This works because we don't need absolute coordinates (just distance).
		//       If absolute coords needed then must properly account for scale pivot
		FVector RescaleSourcePropV = InvSourceScale * SourcePropCompV;
		FVector RescaleSourceBodyV = InvSourceScale * SourceBodyCompV;

		double SourceVDist = FVector::Distance(RescaleSourcePropV, RescaleSourceBodyV);
		double LinearWeight = FMath::Clamp(1.0 - (SourceVDist / Settings.DistanceThreshold), 0.0, 1.0);
		double Weight = GetDistanceWeight(Settings.DistanceWeightMode, SourceVDist, Settings.DistanceThreshold, Settings.DistHalfPointLambda);

		// Compute Source pair verts as represented in alternate pair body primary(_P) or other(_O) local bake space
		FVector SourcePropV_B = InverseBodyTransform(SourceBodyTfm, SourcePropCompV);
		FVector SourceBodyV_P = SourcePropTfm.InverseTransformPosition(SourceBodyCompV);
		
		//---------------------------------
		// Get verts retargeted using physics body oriented scale
		int32 TargetPropPoseIdx = CacheTargetSkelIndices[TargetPropBone];
		int32 TargetBodyPoseIdx = CacheTargetSkelIndices[TargetBodyBone];

		FTransform TargetPropBoneDelta = GetTargetBoneDelta(TargetPropBone);

		FTransform TargetPropTfm = TargetPropBoneDelta * TargetPose[TargetPropPoseIdx];
		FBodyTransform TargetBodyTfm;
		ComputeTargetBodyTransform(TargetBodyTfm, TargetBodyBone, TargetPose[TargetBodyPoseIdx]);

		// Compute prop vert represented in component space relative to prop (_P) and body (_B)
		FVector TargetPropCompV_P = TargetPropTfm.TransformPosition(SourceBoneBakePropV);
		FVector TargetPropCompV_B = ApplyBodyTransform(TargetBodyTfm, SourcePropV_B);

		// Compute body vert represented in component space relative to prop (_P) and body (_B)
		FVector TargetBodyCompV_B = ApplyBodyTransform(TargetBodyTfm, SourceBoneBakeBodyV);
		FVector TargetBodyCompV_P = TargetPropTfm.TransformPosition(SourceBodyV_P);
		
		// These are the target midpoint positions used for spring constraints in PhysicalIK
		FVector TargetSpring_P = FMath::Lerp(TargetPropCompV_P,TargetBodyCompV_P, RetargetSpringAlpha);
		FVector TargetSpring_B = FMath::Lerp(TargetPropCompV_B, TargetBodyCompV_B, RetargetSpringAlpha);

		// Offset vector (Body -> Prop) to Lerp for dist alpha
		FVector TargetOffsetVec = TargetSpring_P - TargetSpring_B;

		FVector TargetPropBoneLoc = TargetPose[TargetPropPoseIdx].GetLocation();
		FVector TargetBodyBoneLoc = TargetPose[TargetBodyPoseIdx].GetLocation();
		
		FDoubleInterval FeasibleRange_P(0.0, 1.0);
		FDoubleInterval FeasibleRange_B(0.0, 1.0);
		UpdateCacheChainInfo(TargetPose, TargetBodyBone);
		if (bBodyMovable)
		{
			ComputeFeasibilityRange(FeasibleRange_B, TargetBodyBone, TargetBodyBoneLoc, TargetOffsetVec);
		}

		double ContactBlendWeight = (Settings.bBidirectionalDistanceContactBlend) ? LinearWeight : 0.5;
		double FeasibilityWeight_B =  FMath::Lerp(FeasibleRange_B.Min, FeasibleRange_B.Max, ContactBlendWeight);
		double FeasibilityWeight_P = FMath::Lerp(FeasibleRange_P.Min, FeasibleRange_P.Max, ContactBlendWeight);
		
		FVector WeightedTargetOffset_B = TargetOffsetVec * (Settings.bMultiBoneSolve? 0.5*(FeasibilityWeight_B+FeasibilityWeight_P): FeasibilityWeight_B);
		FVector WeightedTargetOffset_P = -TargetOffsetVec * (Settings.bMultiBoneSolve? 0.5*(FeasibilityWeight_B+FeasibilityWeight_P): FeasibilityWeight_P);
		
		FrameBoneConstraints[2*OutPairIdx].ContactOffset = TargetSpring_P - TargetPropBoneLoc;
		FrameBoneConstraints[2*OutPairIdx+1].ContactOffset = TargetSpring_B - TargetBodyBoneLoc;
		FrameBoneConstraints[2*OutPairIdx].Offset = WeightedTargetOffset_P;
		FrameBoneConstraints[2*OutPairIdx+1].Offset = WeightedTargetOffset_B;
		FrameBoneConstraints[2*OutPairIdx].FeasibilityWeight = FeasibilityWeight_P;
		FrameBoneConstraints[2*OutPairIdx+1].FeasibilityWeight = FeasibilityWeight_B;
		FrameBoneConstraints[2*OutPairIdx].Weight = Weight;
		FrameBoneConstraints[2*OutPairIdx+1].Weight = Weight;
		FrameBoneConstraints[2*OutPairIdx].SourceDist = SourceVDist;
		FrameBoneConstraints[2*OutPairIdx+1].SourceDist = SourceVDist;
		FrameBoneConstraints[2*OutPairIdx].bAmplifyCloseWeight = bAmplify;
		FrameBoneConstraints[2*OutPairIdx+1].bAmplifyCloseWeight = bAmplify;

#if WITH_EDITOR
		// Debug draw verts
		if (Settings.bDebugDraw)
		{
			DebugInfo.SourcePairVerts.Add({SourcePropCompV, SourceBodyCompV, false, FVector::ZeroVector, FVector::ZeroVector, Weight});
			if (Settings.DebugFullRetargetPairBones.Contains(TargetPropBone))
			{
				DebugInfo.TargetPairVerts.Add({TargetPropCompV_P, TargetBodyCompV_B, true,TargetBodyCompV_P, TargetPropCompV_B, Weight});
			}
			else
			{
				DebugInfo.TargetPairVerts.Add({TargetPropCompV_P, TargetBodyCompV_B, false, FVector::ZeroVector, FVector::ZeroVector, Weight});
			}
			
			DebugPairSpaces[2*OutPairIdx] = FDebugRelativeTargetPairSpace{TargetPropCompV_P, TargetPropCompV_P-TargetOffsetVec, FeasibilityWeight_P, FeasibleRange_P};
			DebugPairSpaces[2*OutPairIdx+1] = FDebugRelativeTargetPairSpace{TargetBodyCompV_B, TargetBodyCompV_B+TargetOffsetVec, FeasibilityWeight_B, FeasibleRange_B};
		}
#endif //WITH_EDITOR
	}
}

void FIKRetargetRelativeIKOp::AmplifyImportanceWeights()
{
	TArray<int32> AmplifyEffectIdx;
	double MaxEffectWeight = 0.0;
	AmplifyEffectIdx.Reset(CacheSourceBodyEffectVertIdx.Num());
	for (int32 EffectIdx = 0; EffectIdx < CacheSourceBodyEffectVertIdx.Num(); ++EffectIdx)
	{
		TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
		if (EffectIndices.IsEmpty())
		{
			continue;
		}

		FName SourceBoneName = CacheSourceEffectBones[EffectIdx];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (!CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}
		
		bool bAmplifyBone = false;
		double TotalWeight = 0.0;
		for (int Index : EffectIndices)
		{
			const FRelativeIKFrameBoneConstraint& BoneConstraint = FrameBoneConstraints[Index];
			if (FMath::IsNearlyZero(BoneConstraint.Weight))
			{
				continue;
			}
			
			if (BoneConstraint.bAmplifyCloseWeight && BoneConstraint.SourceDist <= Settings.AmplifyCloseThreshold)
			{
				bAmplifyBone = true;
				continue;
			}
			
			double TargetWeight = BoneConstraint.Weight;
			TotalWeight += TargetWeight;
		}
		
		if (bAmplifyBone)
		{
			AmplifyEffectIdx.Add(EffectIdx);
		}
		
		// TODO: Is applying max across all amplify weights reasonable?
		MaxEffectWeight = FMath::Max(MaxEffectWeight, TotalWeight);
	}
	
	const double AmpWeight = MaxEffectWeight * Settings.AmplifyScale;
	for (int32 EffectIdx : AmplifyEffectIdx)
	{
		TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
		
		for (int Index : EffectIndices)
		{
			FRelativeIKFrameBoneConstraint& BoneConstraint = FrameBoneConstraints[Index];
			if (!BoneConstraint.bAmplifyCloseWeight || BoneConstraint.SourceDist > Settings.AmplifyCloseThreshold)
			{
				continue;
			}
			
			const double Weight = BoneConstraint.Weight;
			if (BoneConstraint.Weight >= AmpWeight)
			{
				continue;
			}
			
			double BlendAlpha = FMath::Clamp((Settings.AmplifyCloseThreshold - BoneConstraint.SourceDist) / (Settings.AmplifyCloseThreshold - Settings.AmplifyCloseDistance), 0.0, 1.0);
			double BlendWeight = FMath::Lerp(BoneConstraint.Weight, AmpWeight, BlendAlpha);
			
			BoneConstraint.Weight = BlendWeight;
		}
	}
}

void FIKRetargetRelativeIKOp::InitializePinData(const FIKRetargetProcessor& Processor)
{
	// get skeletons we are copying from/to
	const FRetargetSkeleton& SourceSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Source);
	const FRetargetSkeleton& TargetSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Target);

	// get retarget pose of source and target
	const TArray<FTransform>& SourceRetargetRefPose = SourceSkeleton.RetargetPoses.GetGlobalRetargetPose();
	const TArray<FTransform>& TargetRetargeRefPose = TargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
	
	CachePinBoneData.Reset(Settings.PinPropBones.Num());
	for (int32 SettingId = 0; SettingId < Settings.PinPropBones.Num(); ++SettingId)
	{
		const FPinBoneSettings& PinBoneInfo = Settings.PinPropBones[SettingId];
		FName SourceBoneName = PinBoneInfo.SourceBoneName;
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (TargetBoneName == NAME_None)
		{
			continue;
		}
		
		UpdateSourceBoneMap(SourceBoneName);
		UpdateTargetBoneMap(TargetBoneName);
		
		if (!CacheSourceSkelIndices.Contains(SourceBoneName) || !CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}
		
		// get relevant transforms from the retarget poses
		int32 PropBoneIdx = CacheSourceSkelIndices[SourceBoneName];
		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
		
		FTransform PropBoneParentRefPoseGlobal = GetParentTransform(SourceSkeleton, PropBoneIdx, SourceRetargetRefPose);
		FTransform PropBoneRefPoseGlobal = SourceRetargetRefPose[PropBoneIdx];
		
		FTransform TargetBoneParentRefPoseGlobal = GetParentTransform(TargetSkeleton, TargetBoneIdx, TargetRetargeRefPose);
		FTransform TargetBoneRefPoseGlobal = TargetRetargeRefPose[TargetBoneIdx];
		
		FRelativeIKPinBoneData PinData;
		PinData.TranslationMode = PinBoneInfo.TranslationMode;
		PinData.RotationMode = PinBoneInfo.RotationMode;
		
		PinData.SourceBone = SourceBoneName;
		PinData.TargetBone = TargetBoneName;
		PinData.PropScalar = PinBoneInfo.PropScalar;
		
		PinData.SourceTargetOffsetTfm = PropBoneRefPoseGlobal.GetRelativeTransform(TargetBoneRefPoseGlobal);
		PinData.SourceLocalRefTfm = PropBoneParentRefPoseGlobal.GetRelativeTransform(PropBoneRefPoseGlobal);
		PinData.TargetLocalRefTfm = TargetBoneParentRefPoseGlobal.GetRelativeTransform(TargetBoneRefPoseGlobal);
		PinData.RefRotDelta = TargetBoneRefPoseGlobal.GetRotation() * PropBoneRefPoseGlobal.GetRotation().Inverse();
		
		CachePinBoneData.Add(PinData);
	}
}

void FIKRetargetRelativeIKOp::InitializeIntersectionData(const FIKRetargetProcessor& Processor)
{
	// get skeletons we are copying from/to
	const FRetargetSkeleton& SourceSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Source);
	const FRetargetSkeleton& TargetSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Target);
	
	for (FName IntersectBody: Settings.IntersectBodies)
	{
		FName SourceBoneName = IntersectBody;
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (TargetBoneName == NAME_None)
		{
			continue;
		}
		
		UpdateSourceBoneMap(SourceBoneName);
		UpdateTargetBoneMap(TargetBoneName);
	}
}

void FIKRetargetRelativeIKOp::CachePinBoneTransforms(
	const FIKRetargetProcessor& Processor,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	CachePinTargetPropTransforms.SetNumZeroed(CachePinSourcePropBones.Num());
	for (FRelativeIKPinBoneData& PinData : CachePinBoneData)
	{
		FName SourcePropBone = PinData.SourceBone;
		// Will calculate non-cached pins in ApplyAfterParent
		int32 CachePropIdx = CachePinSourcePropBones.Find(SourcePropBone);
		if (CachePropIdx == INDEX_NONE)
		{
			continue;
		}
		
		FName TargetPropBone = ApplyBodyMap(SourcePropBone);
		if (!CacheSourceSkelIndices.Contains(SourcePropBone) || !CacheTargetSkelIndices.Contains(TargetPropBone))
		{
			continue;
		}
		
		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetPropBone];
		FTransform OutGlobalTransform = ComputePinBoneTransform(PinData, Processor, InSourceGlobalPose, OutTargetGlobalPose);
		if (!Settings.bSolveWithPropScale)
		{
			PinData.PropScalar = Settings.PinPropBones[CachePropId2PinTargetPropSettingId[CachePropIdx]].PropScalar;
		}
		OutGlobalTransform.SetScale3D(FVector(PinData.PropScalar));
		OutTargetGlobalPose[TargetBoneIdx] = OutGlobalTransform;
		CachePinTargetPropTransforms[CachePropIdx] = OutGlobalTransform;
	}
}

FTransform FIKRetargetRelativeIKOp::ComputePinBoneTransform(
		const FRelativeIKPinBoneData& PinData,
		const FIKRetargetProcessor& Processor,
		const TArray<FTransform>& InSourceGlobalPose,
		const TArray<FTransform>& OutTargetGlobalPose)
{
	int32 SourceBoneIdx = CacheSourceSkelIndices[PinData.SourceBone];
	int32 TargetBoneIdx = CacheTargetSkelIndices[PinData.TargetBone];
	
	const FRetargetSkeleton& SourceSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Source);
	const FRetargetSkeleton& TargetSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Target);
	
	const FTransform& SourceTransform = InSourceGlobalPose[SourceBoneIdx]; 
	FTransform TargetTransform = SourceTransform;
	
	// get the current translational offset of BoneToCopyFrom relative to it's parent
	auto GetSourceLocalBoneVector = [SourceSkeleton, SourceBoneIdx, InSourceGlobalPose]()
	{
		const FTransform& SourceTransform = InSourceGlobalPose[SourceBoneIdx];
		const FTransform& ParentTransform = GetParentTransform(SourceSkeleton, SourceBoneIdx, InSourceGlobalPose);
		return SourceTransform.GetTranslation() - ParentTransform.GetTranslation();
	};
	
	// generate translational offset
	switch (PinData.TranslationMode)
	{
	case EPinBoneTranslationMode::CopyGlobalPosition:
		{
			TargetTransform.SetTranslation(SourceTransform.GetTranslation());
			break;
		}
		
	case EPinBoneTranslationMode::CopyGlobalPositionAndMaintainOffset:
		{
			TargetTransform.SetTranslation((PinData.SourceTargetOffsetTfm * SourceTransform).GetTranslation());
			break;
		}
		
	case EPinBoneTranslationMode::CopyLocalPosition:
		{
			const FVector SourceLocalVector = GetSourceLocalBoneVector();
			const FTransform& TargetParentTfm = GetParentTransform(TargetSkeleton, TargetBoneIdx, OutTargetGlobalPose);
			TargetTransform.SetTranslation(TargetParentTfm.GetTranslation() + SourceLocalVector);
			break;
		}
		
	case EPinBoneTranslationMode::CopyLocalPositionRelativeOffset:
		{
			const FVector SourceLocalVector = GetSourceLocalBoneVector();
			double SourceBoneLength;
			FVector SourceTranslationDir;
			SourceLocalVector.ToDirectionAndLength(SourceTranslationDir, SourceBoneLength);
			
			const double RestPoseLengthDifference = PinData.TargetLocalRefTfm.GetTranslation().Size() - PinData.SourceLocalRefTfm.GetTranslation().Size();
			const FVector SourceRelativeOffset = SourceTranslationDir * (SourceBoneLength + RestPoseLengthDifference);
		
			const FTransform& ParentOfBoneToCopyToCurrent = GetParentTransform(TargetSkeleton, TargetBoneIdx, OutTargetGlobalPose);
			TargetTransform.SetTranslation(ParentOfBoneToCopyToCurrent.GetTranslation() + SourceRelativeOffset);
			break;
		}
		
	case EPinBoneTranslationMode::CopyLocalPositionRelativeScaled:
		{
			const FVector SourceLocalVector = GetSourceLocalBoneVector();
			double SourceBoneLength;
			FVector SourceTranslationDir;
			SourceLocalVector.ToDirectionAndLength(SourceTranslationDir, SourceBoneLength);
			
			const double SourceRefBoneLength = PinData.SourceLocalRefTfm.GetTranslation().Size();
			const double TargetRefBoneLength = PinData.TargetLocalRefTfm.GetTranslation().Size();
			const double BoneLengthScaleFactor = (!FMath::IsNearlyZero(SourceRefBoneLength)) ? TargetRefBoneLength / SourceRefBoneLength : 0.0;
			
			const FVector BoneToCopyFromRelativeOffset = SourceTranslationDir * (SourceBoneLength * BoneLengthScaleFactor);
		
			const FTransform& ParentOfBoneToCopyToCurrent = GetParentTransform(TargetSkeleton, TargetBoneIdx, OutTargetGlobalPose);
			TargetTransform.SetTranslation(ParentOfBoneToCopyToCurrent.GetTranslation() + BoneToCopyFromRelativeOffset);
			break;
		}
		
	default:
		ensureAlwaysMsgf(false, TEXT("Missing translation offset mode."));
		break;
	}

	// generate rotational offset
	switch (PinData.RotationMode)
	{
	case EPinBoneRotationMode::CopyGlobalRotation:
		{
			TargetTransform.SetRotation(SourceTransform.GetRotation());
			break;
		}
	case EPinBoneRotationMode::MaintainOffsetFromBoneToCopyFrom:
		{
			// apply "copy from" rotation plus delta
			TargetTransform.SetRotation(PinData.RefRotDelta * SourceTransform.GetRotation());
			break;
		}
		
	default:
		ensureAlwaysMsgf(false, TEXT("Missing rotation offset mode."));
		break;
	}
	
	return TargetTransform;
}

const FTransform& FIKRetargetRelativeIKOp::GetParentTransform(const FRetargetSkeleton& Skeleton, const int32 BoneIndex, const TArray<FTransform>& InPose)
{
	int32 ParentIdx = Skeleton.GetParentIndex(BoneIndex);
	if (ParentIdx == INDEX_NONE)
	{
		return FTransform::Identity;
	}
	
	return InPose[ParentIdx];
}

void FIKRetargetRelativeIKOp::ApplyAverageBodyPairTargets(
	TArray<FIKRigGoal>& OutIKGoals,
	const TArray<FTransform>& TargetPose,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	for (int32 EffectIdx = 0; EffectIdx < CacheSourceBodyEffectVertIdx.Num(); ++EffectIdx)
	{
		TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
		if (EffectIndices.IsEmpty())
		{
			continue;
		}

		FName SourceBoneName = CacheSourceEffectBones[EffectIdx];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (!CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}

		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
		const FTransform& TargetBonePose = TargetPose[TargetBoneIdx];
		FVector BoneLoc = TargetBonePose.GetLocation();

		FVector TargetOffset;
		FVector TotalWeight = ComputeTargetWeightedOffset(TargetOffset, EffectIndices, BoneLoc, DebugInfo, DebugPairSpaces);
		if (FMath::IsNearlyZero(TotalWeight.GetMax()))
		{
			continue;
		}
		
		UpdateTargetIKGoal(OutIKGoals, TargetBoneName, TotalWeight, TargetOffset, TargetBonePose);
		UpdatePropPinTransform(SourceBoneName, TotalWeight, TargetOffset, TargetBonePose);
	}
}

void FIKRetargetRelativeIKOp::ApplyMatrixBodyPairTargets(
	TArray<FIKRigGoal>& OutIKGoals,
	const TArray<FTransform>& TargetPose,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	TArray<int32> OutIndexMap;
	TArray<FName> OutBoneNames;
	TArray<FVector> OutBoneLocations;
	
	OutIndexMap.Reserve(CacheSourceBodyEffectVertIdx.Num());
	OutBoneNames.Reserve(CacheSourceBodyEffectVertIdx.Num());
	OutBoneLocations.Reserve(CacheSourceBodyEffectVertIdx.Num());
	for (int EffectIdx = 0; EffectIdx < CacheSourceBodyEffectVertIdx.Num(); ++EffectIdx)
	{
		TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
		if (EffectIndices.IsEmpty())
		{
			continue;
		}

		FName SourceBoneName = CacheSourceEffectBones[EffectIdx];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (!CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}

		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
		const FTransform& TargetBonePose = TargetPose[TargetBoneIdx];
		
		OutIndexMap.Add(EffectIdx);
		OutBoneNames.Add(SourceBoneName);
		OutBoneLocations.Add(TargetBonePose.GetLocation());
	}
	
	// TODO: We need to move all processing to be "output-oriented" (effectors) when we generalize solver!
	if (OutIndexMap.IsEmpty())
	{
		return;
	}
	
	Eigen::MatrixXd BaseWeightMatrix = Eigen::MatrixXd::Identity(OutIndexMap.Num(), OutIndexMap.Num());
	Eigen::MatrixXd OffsetMatrix = Eigen::MatrixXd::Zero(OutIndexMap.Num(), 3);
	Eigen::MatrixXd DeltaWeights = Eigen::MatrixXd::Zero(OutIndexMap.Num(), 3);
	Eigen::VectorXd BaseWeights = Eigen::VectorXd::Zero(OutIndexMap.Num());

	double MaxDelta = 0.0;
	for (int32 RowIdx = 0; RowIdx < OutIndexMap.Num(); ++RowIdx)
	{
		int32 EffectIdx = OutIndexMap[RowIdx];
		TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
		if (EffectIndices.IsEmpty())
		{
			continue;
		}
		
		FVector TotalWeight = ComputeTargetWeightedOffsetRow(OffsetMatrix, RowIdx, OutBoneNames, EffectIndices, OutBoneLocations[RowIdx], DebugInfo, DebugPairSpaces);
		if (FMath::IsNearlyZero(TotalWeight.GetMax()))
		{
			continue;
		}
		
		BaseWeights(RowIdx) = TotalWeight.GetMin();
		DeltaWeights(RowIdx,0) = TotalWeight.X - BaseWeights(RowIdx);
		DeltaWeights(RowIdx,1) = TotalWeight.Y - BaseWeights(RowIdx);
		DeltaWeights(RowIdx,2) = TotalWeight.Z - BaseWeights(RowIdx);
		
		ComputeWeightMatrixRow(BaseWeightMatrix, RowIdx, OutBoneNames, EffectIndices);
		BaseWeightMatrix(RowIdx, RowIdx) = BaseWeights(RowIdx) + Settings.MultiBoneRegularization;
	}
	
	Eigen::MatrixXd SolvedOffset(OutIndexMap.Num(), 3);
	if (FMath::IsNearlyZero(DeltaWeights.maxCoeff()))
	{
		// Eigen::ColPivHouseholderQR<Eigen::MatrixXd> qr = BaseWeightMatrix.colPivHouseholderQr();
		// Eigen::LLT<Eigen::MatrixXd> MatrixSolver = BaseWeightMatrix.llt();
		Eigen::JacobiSVD<Eigen::MatrixXd> MatrixSolver = BaseWeightMatrix.jacobiSvd(Eigen::ComputeThinU | Eigen::ComputeThinV);
		SolvedOffset = MatrixSolver.solve(OffsetMatrix);
	}
	else
	{
		// TODO: Generalize for partial conditions on any/all coordinate sets
		// Eigen::LLT<Eigen::MatrixXd> MatrixSolver = BaseWeightMatrix.llt();
		Eigen::JacobiSVD<Eigen::MatrixXd> MatrixSolver = BaseWeightMatrix.jacobiSvd(Eigen::ComputeThinU | Eigen::ComputeThinV);
		SolvedOffset.leftCols<2>() = MatrixSolver.solve(OffsetMatrix.leftCols<2>());
		
		//TODO: This is kinda a nasty eigen op, could alternatively build this up in since update matrix by add/solve/subtract
		Eigen::MatrixXd UpdatedWeights = BaseWeightMatrix + DeltaWeights.col(2).asDiagonal().toDenseMatrix();
		// MatrixSolver = UpdatedWeights.llt();
		MatrixSolver = UpdatedWeights.jacobiSvd(Eigen::ComputeThinU | Eigen::ComputeThinV);
		SolvedOffset.rightCols<1>() = MatrixSolver.solve(OffsetMatrix.rightCols<1>());
	}
	
	for (int32 RowIdx = 0; RowIdx < OutIndexMap.Num(); ++RowIdx)
	{
		int32 EffectIdx = OutIndexMap[RowIdx];
		FVector TotalWeight = FVector{DeltaWeights(RowIdx, 0), DeltaWeights(RowIdx, 1), DeltaWeights(RowIdx, 2)} + BaseWeights(RowIdx);
		if (FMath::IsNearlyZero(TotalWeight.GetMax()))
		{
			continue;
		}
		
		FName SourceBoneName = CacheSourceEffectBones[EffectIdx];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (!CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}
		
		FVector TargetOffset{
			(!FMath::IsNearlyZero(TotalWeight.X)) ? SolvedOffset(RowIdx, 0) : 0.0,
			(!FMath::IsNearlyZero(TotalWeight.Y)) ? SolvedOffset(RowIdx, 1) : 0.0,
			(!FMath::IsNearlyZero(TotalWeight.Z)) ? SolvedOffset(RowIdx, 2) : 0.0
		};

		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
		const FTransform& TargetBonePose = TargetPose[TargetBoneIdx];
		
		UpdateTargetIKGoal(OutIKGoals, TargetBoneName, TotalWeight, TargetOffset, TargetBonePose);
		UpdatePropPinTransform(SourceBoneName, TotalWeight, TargetOffset, TargetBonePose);
	}
}

void FIKRetargetRelativeIKOp::ApplyMatrixBodyPairTargetsWithPropScale(
	TArray<FIKRigGoal>& OutIKGoals,
	const TArray<FTransform>& TargetPose,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	TArray<int32> OutIndexMap;
	TArray<FName> OutBoneNames;
	TArray<FVector> OutBoneLocations;
	TArray<int32> ScalarIndexMap;
	TArray<FName> ScalarBoneNames;

	ScalarIndexMap.Reserve(CachePinSourcePropBones.Num());
	ScalarBoneNames.Reserve(CachePinSourcePropBones.Num());
	OutIndexMap.Reserve(CacheSourceBodyEffectVertIdx.Num());
	OutBoneNames.Reserve(CacheSourceBodyEffectVertIdx.Num());
	OutBoneLocations.Reserve(CacheSourceBodyEffectVertIdx.Num());
	for (int EffectIdx = 0; EffectIdx < CacheSourceBodyEffectVertIdx.Num(); ++EffectIdx)
	{
		TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
		if (EffectIndices.IsEmpty())
		{
			continue;
		}

		FName SourceBoneName = CacheSourceEffectBones[EffectIdx];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (!CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}

		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
		const FTransform& TargetBonePose = TargetPose[TargetBoneIdx];
		
		int32 OutBoneIdx = OutIndexMap.Num();
		OutIndexMap.Add(EffectIdx);
		OutBoneNames.Add(SourceBoneName);
		OutBoneLocations.Add(TargetBonePose.GetLocation());
		
		int32 PinPropIdx = CachePinSourcePropBones.Find(SourceBoneName);
		if (PinPropIdx != INDEX_NONE && CachePropId2PinBoneDataId[PinPropIdx] != INDEX_NONE)
		{
			ScalarIndexMap.Add(OutBoneIdx);
	        ScalarBoneNames.Add(SourceBoneName);
		}
	}
	
	if (OutIndexMap.IsEmpty())
	{
		return;
	}

	Eigen::MatrixXd WeightMatrix = Eigen::MatrixXd::Zero(3*OutIndexMap.Num()+ScalarIndexMap.Num(), 3*OutIndexMap.Num()+ScalarIndexMap.Num());
	Eigen::MatrixXd OffsetMatrix = Eigen::MatrixXd::Zero(3*OutIndexMap.Num()+ScalarIndexMap.Num(), 1);
	// Eigen::VectorXd BaseWeights = Eigen::VectorXd::Zero(OutIndexMap.Num());

	for (int32 RowIdx = 0; RowIdx < OutIndexMap.Num(); ++RowIdx)
	{
		int32 EffectIdx = OutIndexMap[RowIdx];
		TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
		FVector OffsetLoc = FVector::ZeroVector;
		for (int Index : EffectIndices)
		{
			const FRelativeIKFrameBoneConstraint& BoneConstraint = FrameBoneConstraints[Index];
			if (FMath::IsNearlyZero(BoneConstraint.Weight))
			{
				continue;
			}

			FName EffectBone = CachePairEffectBoneNames[Index];
			int32 OutWeightIdx = OutBoneNames.Find(EffectBone);

			if (BoneConstraint.ConstraintType == ERelativeIKFrameBoneConstraintType::XYZ)
			{
				WeightMatrix(3*RowIdx+0, 3*RowIdx+0) += BoneConstraint.Weight;
				WeightMatrix(3*RowIdx+1, 3*RowIdx+1) += BoneConstraint.Weight;
				WeightMatrix(3*RowIdx+2, 3*RowIdx+2) += BoneConstraint.Weight;
				if (OutWeightIdx != INDEX_NONE)
				{
					WeightMatrix(3*RowIdx+0, 3*OutWeightIdx+0) -= BoneConstraint.Weight;
					WeightMatrix(3*RowIdx+1, 3*OutWeightIdx+1) -= BoneConstraint.Weight;
					WeightMatrix(3*RowIdx+2, 3*OutWeightIdx+2) -= BoneConstraint.Weight;
				}
			}
			else if (BoneConstraint.ConstraintType == ERelativeIKFrameBoneConstraintType::Z)
			{
				WeightMatrix(3*RowIdx+2, 3*RowIdx+2) += BoneConstraint.Weight;
				if (OutWeightIdx != INDEX_NONE)
				{
					WeightMatrix(3*RowIdx+2, 3*OutWeightIdx+2) -= BoneConstraint.Weight;
				}
			}
			
			OffsetLoc += BoneConstraint.Weight * BoneConstraint.Offset;
		}
		
		OffsetMatrix(3*RowIdx+0) = OffsetLoc.X;
		OffsetMatrix(3*RowIdx+1) = OffsetLoc.Y;
		OffsetMatrix(3*RowIdx+2) = OffsetLoc.Z;
	}
	WeightMatrix.diagonal().array() += Settings.MultiBoneRegularization;

	for (int32 ScalarIdx = 0; ScalarIdx < ScalarIndexMap.Num(); ++ScalarIdx)
	{
		int32 RowIdx = 3*OutIndexMap.Num() + ScalarIdx;
		WeightMatrix(RowIdx, RowIdx) = Settings.PropScalarDamping;
		int32 OutBoneIdx = ScalarIndexMap[ScalarIdx];
		WeightMatrix(3*OutBoneIdx+0, 3*OutBoneIdx+0) += Settings.PropScalarBoneRegularization-Settings.MultiBoneRegularization;
		WeightMatrix(3*OutBoneIdx+1, 3*OutBoneIdx+1) += Settings.PropScalarBoneRegularization-Settings.MultiBoneRegularization;
		WeightMatrix(3*OutBoneIdx+2, 3*OutBoneIdx+2) += Settings.PropScalarBoneRegularization-Settings.MultiBoneRegularization;
		int32 EffectIdx = OutIndexMap[OutBoneIdx];
		TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
		FVector ScalarOffsetLoc = FVector::ZeroVector;
		
		int32 CachePropIdx = CachePinSourcePropBones.Find(ScalarBoneNames[ScalarIdx]);
		double PropScalar = CachePinBoneData[CachePropId2PinBoneDataId[CachePropIdx]].PropScalar;
		
		for (int Index : EffectIndices)
		{
			const FRelativeIKFrameBoneConstraint& BoneConstraint = FrameBoneConstraints[Index];
			if (FMath::IsNearlyZero(BoneConstraint.Weight))
			{
				continue;
			}
			int IndexOpp;
			if (Index%2==0)
			{
				IndexOpp = Index + 1;
			}
			else
			{
				IndexOpp = Index - 1;
			}
			const FRelativeIKFrameBoneConstraint& BoneConstraintOpp = FrameBoneConstraints[IndexOpp];
			
			FName EffectBone = CachePairEffectBoneNames[Index];
			int32 OutWeightIdx = OutBoneNames.Find(EffectBone);
			int32 OutScaleIdx = ScalarBoneNames.Find(EffectBone);
			
			FVector ContactOffset = FVector::ZeroVector;
			if (!FMath::IsNearlyZero(PropScalar))
			{
				ContactOffset = BoneConstraint.ContactOffset/PropScalar;
			}
			
			if (BoneConstraint.ConstraintType == ERelativeIKFrameBoneConstraintType::XYZ)
			{
				
				WeightMatrix(RowIdx, RowIdx) += BoneConstraint.Weight * ContactOffset.X * ContactOffset.X;
				WeightMatrix(RowIdx, 3*OutBoneIdx+0) += BoneConstraint.Weight * ContactOffset.X;
				WeightMatrix(RowIdx, RowIdx) += BoneConstraint.Weight * ContactOffset.Y * ContactOffset.Y;
				WeightMatrix(RowIdx, 3*OutBoneIdx+1) += BoneConstraint.Weight * ContactOffset.Y;
				WeightMatrix(RowIdx, RowIdx) += BoneConstraint.Weight * ContactOffset.Z * ContactOffset.Z;
				WeightMatrix(RowIdx, 3*OutBoneIdx+2) += BoneConstraint.Weight * ContactOffset.Z;
				if (OutWeightIdx != INDEX_NONE)
				{
					WeightMatrix(RowIdx, 3*OutWeightIdx+0) -= BoneConstraint.Weight * ContactOffset.X;
					WeightMatrix(RowIdx, 3*OutWeightIdx+1) -= BoneConstraint.Weight * ContactOffset.Y;
					WeightMatrix(RowIdx, 3*OutWeightIdx+2) -= BoneConstraint.Weight * ContactOffset.Z;
				}
				if (OutScaleIdx != INDEX_NONE)
				{
					int32 OutCachePropIdx = CachePinSourcePropBones.Find(ScalarBoneNames[OutScaleIdx]);
					double OutPropScalar = CachePinBoneData[CachePropId2PinBoneDataId[OutCachePropIdx]].PropScalar;
					OutScaleIdx = 3*OutIndexMap.Num() + OutScaleIdx;
					if (!FMath::IsNearlyZero(OutPropScalar))
					{
						WeightMatrix(RowIdx, OutScaleIdx) -= 0.5*BoneConstraint.Weight * ContactOffset.X * BoneConstraintOpp.ContactOffset.X/OutPropScalar;
						WeightMatrix(RowIdx, OutScaleIdx) -= 0.5*BoneConstraint.Weight * ContactOffset.Y * BoneConstraintOpp.ContactOffset.Y/OutPropScalar;
						WeightMatrix(RowIdx, OutScaleIdx) -= 0.5*BoneConstraint.Weight * ContactOffset.Z * BoneConstraintOpp.ContactOffset.Z/OutPropScalar;
					}
				}
			}
			else if (BoneConstraint.ConstraintType == ERelativeIKFrameBoneConstraintType::Z)
			{
				WeightMatrix(RowIdx, RowIdx) += BoneConstraint.Weight * ContactOffset.Z * ContactOffset.Z;
				WeightMatrix(RowIdx, 3*OutBoneIdx+2) += BoneConstraint.Weight * ContactOffset.Z;
				if (OutWeightIdx != INDEX_NONE)
				{
					WeightMatrix(RowIdx, 3*OutWeightIdx+2) -= BoneConstraint.Weight * ContactOffset.Z;
				}
				if (OutScaleIdx != INDEX_NONE)
				{
					int32 OutCachePropIdx = CachePinSourcePropBones.Find(ScalarBoneNames[OutScaleIdx]);
					double OutPropScalar = CachePinBoneData[CachePropId2PinBoneDataId[OutCachePropIdx]].PropScalar;
					OutScaleIdx = 3*OutIndexMap.Num() + OutScaleIdx;
					if (!FMath::IsNearlyZero(OutPropScalar))
					{
						WeightMatrix(RowIdx, OutScaleIdx) -= 0.5*BoneConstraint.Weight * ContactOffset.Z * BoneConstraintOpp.ContactOffset.Z/OutPropScalar;
					}
				}	
			}

			ScalarOffsetLoc += BoneConstraint.Weight * BoneConstraint.Offset * ContactOffset;
		}
		
		OffsetMatrix(RowIdx) += ScalarOffsetLoc.X + ScalarOffsetLoc.Y + ScalarOffsetLoc.Z + Settings.PropScalarDamping*(Settings.PinPropBones[CachePropId2PinTargetPropSettingId[CachePropIdx]].PropScalar-PropScalar);
	}

	WeightMatrix = WeightMatrix.selfadjointView<Eigen::Lower>();

	Eigen::MatrixXd SolvedOffset(3*OutIndexMap.Num() + ScalarIndexMap.Num(), 1);
	Eigen::LLT<Eigen::MatrixXd> MatrixSolver = WeightMatrix.llt();
	SolvedOffset = MatrixSolver.solve(OffsetMatrix);

	for (int32 ScalarIdx = 0; ScalarIdx < ScalarIndexMap.Num(); ++ScalarIdx)
	{
		int32 RowIdx = 3*OutIndexMap.Num() + ScalarIdx;
		int32 OutBoneIdx = ScalarIndexMap[ScalarIdx];
		int32 EffectIdx = OutIndexMap[OutBoneIdx];
		if (FMath::IsNearlyZero(WeightMatrix(RowIdx, RowIdx)))
		{
			continue;
		}
		
		FName SourceBoneName = CacheSourceEffectBones[EffectIdx];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (!CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}
		
		int32 CachePropIdx = CachePinSourcePropBones.Find(SourceBoneName);
		if (CachePropIdx == INDEX_NONE || CachePropId2PinBoneDataId[CachePropIdx] == INDEX_NONE)
		{
			continue;
		}
		
		double &PropScalar = CachePinBoneData[CachePropId2PinBoneDataId[CachePropIdx]].PropScalar;
		if (Settings.bDebugDraw && Settings.bDryRun)
		{
			PropScalar = Settings.PinPropBones[CachePropId2PinTargetPropSettingId[CachePropIdx]].PropScalar;
			continue;
		}
		// PropScalar = FMath::Lerp(PropScalar, Settings.PinPropBones[CachePinTargetPropSettingId[CachePropIdx]].PropScalar,FMath::Clamp(Settings.PropScalarLearningRate*5,0,1));
		PropScalar += FMath::Clamp(SolvedOffset(RowIdx),-Settings.PropScalarLearningRate*PropScalar, Settings.PropScalarLearningRate*PropScalar);
		PropScalar = FMath::Clamp(PropScalar, Settings.PinPropBones[CachePropId2PinTargetPropSettingId[CachePropIdx]].PropScalar*(1-Settings.PropScalarRange), Settings.PinPropBones[CachePropId2PinTargetPropSettingId[CachePropIdx]].PropScalar*(1+Settings.PropScalarRange));
		// PropScalar += SolvedOffset(RowIdx);
		CachePinTargetPropTransforms[CachePropIdx].SetScale3D(FVector(PropScalar));
		
		WeightMatrix.row(RowIdx).setZero();
		WeightMatrix.col(RowIdx).setZero();
		WeightMatrix(RowIdx,RowIdx) = 1.0;
		OffsetMatrix(RowIdx) = PropScalar-1;
	}
	
	//Solve again for fixed prop scalar
	MatrixSolver = WeightMatrix.llt();
	SolvedOffset = MatrixSolver.solve(OffsetMatrix);

	for (int32 RowIdx = 0; RowIdx < OutIndexMap.Num(); ++RowIdx)
	{
		int32 EffectIdx = OutIndexMap[RowIdx];
		FVector TotalWeight = FVector{WeightMatrix(3*RowIdx+0, 3*RowIdx+0), WeightMatrix(3*RowIdx+1, 3*RowIdx+1), WeightMatrix(3*RowIdx+2, 3*RowIdx+2)};
		if (FMath::IsNearlyZero(TotalWeight.GetMax()))
		{
			continue;
		}
		
		FName SourceBoneName = CacheSourceEffectBones[EffectIdx];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (!CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}
		
		FVector TargetOffset{
			(!FMath::IsNearlyZero(TotalWeight.X)) ? SolvedOffset(3*RowIdx+0) : 0.0,
			(!FMath::IsNearlyZero(TotalWeight.Y)) ? SolvedOffset(3*RowIdx+1) : 0.0,
			(!FMath::IsNearlyZero(TotalWeight.Z)) ? SolvedOffset(3*RowIdx+2) : 0.0
		};

		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
		const FTransform& TargetBonePose = TargetPose[TargetBoneIdx];
		
		UpdateTargetIKGoal(OutIKGoals, TargetBoneName, TotalWeight, TargetOffset, TargetBonePose);
		UpdatePropPinTransform(SourceBoneName, TotalWeight, TargetOffset, TargetBonePose);
	}
}

void FIKRetargetRelativeIKOp::ApplyIntersectPushOut(
	TArray<FIKRigGoal>& OutIKGoals,
	const TArray<FTransform>& TargetPose,
	FDebugRelativeIKDrawInfo& DebugInfo)
{
	const UPhysicsAsset* TargetIntersectPhysicsAsset = TargetIntersectOpPhysicsAsset;
	TArray<int32> IKGoalMap;
	TArray<FName> IKGoalSourceNames;
	TArray<int32> IKGoalCollisionIdx;
	IKGoalMap.Reserve(CacheSourceBodyEffectVertIdx.Num());
	IKGoalSourceNames.Reserve(CacheSourceBodyEffectVertIdx.Num());
	IKGoalCollisionIdx.Reserve(CacheSourceBodyEffectVertIdx.Num());
	for (int EffectIdx = 0; EffectIdx < CacheSourceBodyEffectVertIdx.Num(); ++EffectIdx)
	{
		TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
		if (EffectIndices.IsEmpty())
		{
			continue;
		}

		FName SourceBoneName = CacheSourceEffectBones[EffectIdx];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (!CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}
		
		FIKRigGoal* IKGoal = OutIKGoals.FindByPredicate([TargetBoneName](const FIKRigGoal& Other){return Other.BoneName == TargetBoneName;});
		if (IKGoal)
		{
			int32 CheckCollisionIdx = TargetIntersectPhysicsAsset->FindBodyIndex(IKGoal->BoneName);
			if (CheckCollisionIdx == INDEX_NONE)
			{
				continue;
			}
			IKGoalMap.Add(EffectIdx);
			IKGoalSourceNames.Add(SourceBoneName);
			IKGoalCollisionIdx.Add(CheckCollisionIdx);
			continue;
		}
		
		int32 PinPropIdx = CachePinSourcePropBones.Find(SourceBoneName);
		if (PinPropIdx != INDEX_NONE && CachePropId2PinBoneDataId[PinPropIdx] != INDEX_NONE)
		{
			IKGoalMap.Add(EffectIdx);
			IKGoalSourceNames.Add(SourceBoneName);
			IKGoalCollisionIdx.Add(INDEX_NONE);
			continue;
		}
	}
	
	TArray<TArray<int32>> GluedBodies;
	TArray<FVector> GluedBodyDelta;
	TArray<int32> IKGoal2GluedBodyIdx;
	TArray<bool> IKGoalVisited;
	TArray<int32> DfsStack;
	DfsStack.Reserve(IKGoalMap.Num());
	IKGoalVisited.Init(false, IKGoalMap.Num());
	IKGoal2GluedBodyIdx.Init(INDEX_NONE, IKGoalMap.Num());
	for (int32 IkGoalIdx0 = 0; IkGoalIdx0 < IKGoalMap.Num(); ++IkGoalIdx0)
	{
		DfsStack.Push(IkGoalIdx0);
		while (!DfsStack.IsEmpty())
		{
			int32 IkGoalIdx = DfsStack.Pop();
			if (IKGoalVisited[IkGoalIdx])
			{
				continue;
			}
			IKGoalVisited[IkGoalIdx] = true;
			
			int32& GluedBodyIdx = IKGoal2GluedBodyIdx[IkGoalIdx];
			if (GluedBodyIdx == INDEX_NONE)
			{
				GluedBodies.AddDefaulted();
				GluedBodyIdx = GluedBodies.Num() - 1;
			}
			TArray<int32>& GluedBody = GluedBodies[GluedBodyIdx];
			GluedBody.Add(IkGoalIdx);
			int32 EffectIdx = IKGoalMap[IkGoalIdx];
			const TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[EffectIdx];
			for (int Index : EffectIndices)
			{
				const FRelativeIKFrameBoneConstraint& BoneConstraint = FrameBoneConstraints[Index];
				if (FMath::IsNearlyZero(BoneConstraint.Weight))
				{
					continue;
				}

				FName EffectBone = CachePairEffectBoneNames[Index];
				int32 IkGoalIdx2 = IKGoalSourceNames.Find(EffectBone);
			
				if (IkGoalIdx2 != INDEX_NONE)
				{
					if (BoneConstraint.SourceDist<Settings.GlueThreshold)
					{
						DfsStack.Push(IkGoalIdx2);
						IKGoal2GluedBodyIdx[IkGoalIdx2] = GluedBodyIdx;
					}
				}
			}
		}
	}
	
	GluedBodyDelta.SetNumZeroed(GluedBodies.Num());
	for (int32 GlueBodyIdx=0; GlueBodyIdx<GluedBodies.Num(); ++GlueBodyIdx)
	{
		const TArray<int32>& GluedBody = GluedBodies[GlueBodyIdx];
		FVector& IntersectDelta = GluedBodyDelta[GlueBodyIdx];
		for (int32 IkGoalIdx: GluedBody)
		{
			int32 CollisionProxyIndex = IKGoalCollisionIdx[IkGoalIdx];
			FName SourceBoneName = IKGoalSourceNames[IkGoalIdx];
			FName TargetBoneName = ApplyBodyMap(SourceBoneName);
			int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
			const FTransform& TargetBonePose = TargetPose[TargetBoneIdx];
			
			FVector UpdateDelta = FVector::ZeroVector;
			if (CollisionProxyIndex == INDEX_NONE) // Props
			{
				int32 PinPropIdx = CachePinSourcePropBones.Find(SourceBoneName);
				if (!Settings.PinPropBones[CachePropId2PinBoneDataId[PinPropIdx]].PropSkeletalMeshAsset || !Settings.PinPropBones[CachePropId2PinBoneDataId[PinPropIdx]].PropSkeletalMeshAsset->GetPhysicsAsset())
				{
					continue;
				}
				FTransform CollisionProxyTfm = CachePinTargetPropTransforms[PinPropIdx];
				CollisionProxyTfm.SetTranslation(CollisionProxyTfm.GetTranslation()+IntersectDelta);
				const UPhysicsAsset* PropPhysicsAsset = Settings.PinPropBones[CachePropId2PinBoneDataId[PinPropIdx]].PropSkeletalMeshAsset->GetPhysicsAsset();
				const FTransform PropAttachTransform = OffsetTransformsForSourceBones(SourceBoneName);
				const UAnimSequence* PropAnimation = Settings.PinPropBones[CachePropId2PinBoneDataId[PinPropIdx]].PropAnimSequence;
				if (PropAnimation)
				{
					float TestPropAnimLength = PropAnimation->GetPlayLength();
					float TestPropPlayhead = FMath::Max(FMath::Fmod(AnimSeqPlayHead, TestPropAnimLength), 0.0f);
					if (TestPropPlayhead<0.f)
					{
						continue;
					}
					TArray<FName> OutPropBones;
					TArray<FTransform> OutPropPose;
					GetAnimSeqFramePose(PropAnimation, TestPropPlayhead, OutPropBones, OutPropPose);
					FTransform UpdateTfm = CollisionProxyTfm;
					for (int32 PropBodyIdx=0; PropBodyIdx<PropPhysicsAsset->SkeletalBodySetups.Num(); ++PropBodyIdx)
					{
						const FKAggregateGeom* CollisionProxyAggGeom = &PropPhysicsAsset->SkeletalBodySetups[PropBodyIdx]->AggGeom;
						const FName PropBoneName = PropPhysicsAsset->SkeletalBodySetups[PropBodyIdx]->BoneName;
						int32 OutPropBonesIdx = OutPropBones.Find(PropBoneName);
						if (OutPropBonesIdx == INDEX_NONE)
						{
							continue;
						}
						FTransform PropBoneTfm = OutPropPose[OutPropBonesIdx];
						for (int j = 0; j < Settings.IntersectBodies.Num(); ++j)
						{
							FName BodyName = Settings.IntersectBodies[j];
							if (!CacheTargetSkelIndices.Contains(BodyName))
							{
								continue;
							}
							int32 BodyPoseIdx = CacheTargetSkelIndices[BodyName];
							if (BodyPoseIdx == INDEX_NONE)
							{
								continue;
							}
		
							FTransform BodyTfm = TargetPose[BodyPoseIdx];
							const FKAggregateGeom* BodyAggGeom = FPhysShapeUtils::FindBodyAggGeom(TargetIntersectPhysicsAsset, BodyName);
							if (!BodyAggGeom)
							{
								continue;
							}

							FVector DeltaDir = FVector::ZeroVector;
							double Dist = FBodyIntersectUtils::CalcIntersectionPairDelta(PropAttachTransform*PropBoneTfm*UpdateTfm, CollisionProxyAggGeom, BodyTfm, BodyAggGeom, DeltaDir);
							if (Dist > 0.0)
							{
								UpdateDelta += Dist * DeltaDir;
								UpdateTfm.SetTranslation(CollisionProxyTfm.GetTranslation() + UpdateDelta);
							}
						}
					}
				}
				else
				{
					FTransform UpdateTfm = CollisionProxyTfm;
					for (int32 PropBodyIdx=0; PropBodyIdx<PropPhysicsAsset->SkeletalBodySetups.Num(); ++PropBodyIdx)
					{
						const FKAggregateGeom* CollisionProxyAggGeom = &PropPhysicsAsset->SkeletalBodySetups[PropBodyIdx]->AggGeom;
						for (int j = 0; j < Settings.IntersectBodies.Num(); ++j)
						{
							FName BodyName = Settings.IntersectBodies[j];
							if (!CacheTargetSkelIndices.Contains(BodyName))
							{
								continue;
							}
							int32 BodyPoseIdx = CacheTargetSkelIndices[BodyName];
							if (BodyPoseIdx == INDEX_NONE)
							{
								continue;
							}
		
							FTransform BodyTfm = TargetPose[BodyPoseIdx];
							const FKAggregateGeom* BodyAggGeom = FPhysShapeUtils::FindBodyAggGeom(TargetIntersectPhysicsAsset, BodyName);
							if (!BodyAggGeom)
							{
								continue;
							}

							FVector DeltaDir = FVector::ZeroVector;
							double Dist = FBodyIntersectUtils::CalcIntersectionPairDelta(PropAttachTransform*UpdateTfm, CollisionProxyAggGeom, BodyTfm, BodyAggGeom, DeltaDir);
							if (Dist > 0.0)
							{
								UpdateDelta += Dist * DeltaDir;
								UpdateTfm.SetTranslation(CollisionProxyTfm.GetTranslation() + UpdateDelta);
							}
						}
					}
				}
			}
			else
			{
				FIKRigGoal* IKGoal = OutIKGoals.FindByPredicate([TargetBoneName](const FIKRigGoal& Other){return Other.BoneName == TargetBoneName;});
				FVector CompGoalShapeLoc = GoalLocBlendCompSpace(IKGoal, TargetBonePose)+IntersectDelta;
				FTransform CollisionProxyTfm = FTransform(TargetBonePose.Rotator(), CompGoalShapeLoc, TargetBonePose.GetScale3D());
				const FKAggregateGeom* CollisionProxyAggGeom = FPhysShapeUtils::FindBodyAggGeom(TargetIntersectPhysicsAsset, TargetBoneName);
				
				FTransform UpdateTfm = CollisionProxyTfm;
				for (int j = 0; j < Settings.IntersectBodies.Num(); ++j)
				{
					FName BodyName = Settings.IntersectBodies[j];
					if (!CacheTargetSkelIndices.Contains(BodyName))
					{
						continue;
					}
					int32 BodyPoseIdx = CacheTargetSkelIndices[BodyName];
					if (BodyPoseIdx == INDEX_NONE)
					{
						continue;
					}
		
					int32 BodyPhysIdx = TargetIntersectPhysicsAsset->FindBodyIndex(BodyName);
					if (BodyPhysIdx == INDEX_NONE)
					{
						continue;
					}

					// Allow CollisionProxyIndex = -1 to skip collision-enabled check (for prop capsules)
					if (CollisionProxyIndex != INDEX_NONE && !TargetIntersectPhysicsAsset->IsCollisionEnabled(CollisionProxyIndex, BodyPhysIdx))
					{
						continue;
					}
		
					FTransform BodyTfm = TargetPose[BodyPoseIdx];
					const FKAggregateGeom* BodyAggGeom = FPhysShapeUtils::FindBodyAggGeom(TargetIntersectPhysicsAsset, BodyName);
					if (!BodyAggGeom)
					{
						continue;
					}

					FVector DeltaDir = FVector::ZeroVector;
					double Dist = FBodyIntersectUtils::CalcIntersectionPairDelta(UpdateTfm, CollisionProxyAggGeom, BodyTfm, BodyAggGeom, DeltaDir);
					if (Dist > 0.0)
					{
						UpdateDelta += Dist * DeltaDir;
						UpdateTfm.SetTranslation(CollisionProxyTfm.GetTranslation() + UpdateDelta);
					}
				}
			}
			IntersectDelta += UpdateDelta;
		}
	}
	
	bool bSetGoals = !Settings.bDebugDraw || !Settings.bDryRun;
	if (bSetGoals)
	{
		for (int32 GlueBodyIdx=0; GlueBodyIdx<GluedBodies.Num(); ++GlueBodyIdx)
		{
			const TArray<int32>& GluedBody = GluedBodies[GlueBodyIdx];
			const FVector& IntersectDelta = GluedBodyDelta[GlueBodyIdx]*Settings.IntersectOffsetScale;
			for (int32 IkGoalIdx: GluedBody)
			{
				FName SourceBoneName =IKGoalSourceNames[IkGoalIdx];
				FName TargetBoneName = ApplyBodyMap(SourceBoneName);
				int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
				const FTransform& TargetBonePose = TargetPose[TargetBoneIdx];
				int32 CollisionProxyIndex = IKGoalCollisionIdx[IkGoalIdx];
				if (CollisionProxyIndex == INDEX_NONE) // Props
				{
					int32 PinPropIdx = CachePinSourcePropBones.Find(SourceBoneName);
					CachePinTargetPropTransforms[PinPropIdx].SetLocation(CachePinTargetPropTransforms[PinPropIdx].GetLocation()+IntersectDelta);
				}
				else
				{
					FIKRigGoal* IKGoal = OutIKGoals.FindByPredicate([TargetBoneName](const FIKRigGoal& Other){return Other.BoneName == TargetBoneName;});
					FVector CompGoalShapeLoc = GoalLocBlendCompSpace(IKGoal, TargetBonePose)+IntersectDelta;
		    	
					IKGoal->bEnabled = true;
					IKGoal->Position = CompGoalShapeLoc;
					IKGoal->PositionAlpha = 1.0;
					IKGoal->PositionSpace = EIKRigGoalSpace::Component;
				}
			}
		}
	}
}

void FIKRetargetRelativeIKOp::UpdateTargetIKGoal(
	TArray<FIKRigGoal>& OutIKGoals,
	FName TargetBoneName,
	const FVector& TargetWeight,
	const FVector& TargetGoalOffset,
	const FTransform& InTargetBonePose)
{
	FIKRigGoal* IKGoal = OutIKGoals.FindByPredicate([TargetBoneName](const FIKRigGoal& Other){return Other.BoneName == TargetBoneName;});
	if (!IKGoal)
	{
		return;
	}
		
	// TODO: Play with total weight functions to alpha goal
	FVector TargetGoalLoc = InTargetBonePose.GetLocation() + TargetGoalOffset;
	FVector GoalAlpha = FVector::Min(FVector::OneVector, TargetWeight*FMath::Clamp(Settings.ContributionSumWeight, 0.0f, 1.0f));
	FVector PrevGoal = GoalLocBlendCompSpace(IKGoal, InTargetBonePose);
	FVector BlendGoal = FMath::Lerp(PrevGoal, TargetGoalLoc, GoalAlpha);

	// TODO: Remove dry run for release
	bool bSetGoals = !Settings.bDebugDraw || !Settings.bDryRun;
	if (bSetGoals)
	{
		IKGoal->bEnabled = true;
		IKGoal->Position = BlendGoal;
		IKGoal->PositionAlpha = 1.0;
		IKGoal->PositionSpace = EIKRigGoalSpace::Component;
	}
}


void FIKRetargetRelativeIKOp::UpdatePropPinTransform(
	FName SourceBoneName,
	const FVector& TargetWeight,
	const FVector& TargetPropOffset,
	const FTransform& InTargetBonePose)
{
	// Disable prop bone changes if debug drawing and dry run are on
	if (Settings.bDebugDraw && Settings.bDryRun)
	{
		return;
	}
	
	int32 PinPropIdx = CachePinSourcePropBones.Find(SourceBoneName);
	if (PinPropIdx == INDEX_NONE)
	{
		return;
	}
	
	FVector BlendAlpha = FVector::Min(FVector::OneVector, TargetWeight*FMath::Clamp(Settings.ContributionSumWeight, 0.0, 1.0)) * (1.0-Settings.BlendPinsToSource);
	CachePinTargetPropTransforms[PinPropIdx].SetLocation(InTargetBonePose.GetLocation() + BlendAlpha*TargetPropOffset);
}

FVector FIKRetargetRelativeIKOp::ComputeTargetWeightedOffset(
	FVector& OutTargetOffset,
	const TArray<int32>& EffectIndices,
	const FVector& BoneLoc,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	FVector TotalWeight = FVector::ZeroVector;
	OutTargetOffset = FVector::ZeroVector;

#if WITH_EDITOR
	if (Settings.bDebugDraw)
	{
		DebugInfo.TargetGoals.AddDefaulted();
		DebugInfo.TargetGoals.Last().Goal = FVector::ZeroVector;
	}
#endif //WITH_EDITOR
	
	for (int Index : EffectIndices)
	{
		const FRelativeIKFrameBoneConstraint& BoneConstraint = FrameBoneConstraints[Index];
		if (FMath::IsNearlyZero(BoneConstraint.Weight))
		{
			continue;
		}

		FVector TargetOffset = BoneConstraint.Offset;
		FVector TargetWeight = FVector(BoneConstraint.Weight);
		if (BoneConstraint.ConstraintType == ERelativeIKFrameBoneConstraintType::Z)
		{
			TargetWeight = {0,0,BoneConstraint.Weight};
		}

		OutTargetOffset += TargetWeight * TargetOffset;
		TotalWeight += TargetWeight;

#if WITH_EDITOR
		if (Settings.bDebugDraw)
		{
			// TODO: Fixup target debug to show dynamic offset goals (post-solve update?)
			FVector BoneTarget = TargetOffset + BoneLoc;
			DebugInfo.TargetGoals.Last().PairTargets.Add(BoneTarget);
			DebugInfo.PairRetargetInfo.Emplace(DebugPairSpaces[Index]);
		}
#endif //WITH_EDITOR
	}
	
	FVector InvWeight = {
		(FMath::IsNearlyZero(TotalWeight.X) ? 0.0 : 1.0 / TotalWeight.X),
		(FMath::IsNearlyZero(TotalWeight.Y) ? 0.0 : 1.0 / TotalWeight.Y),
		(FMath::IsNearlyZero(TotalWeight.Z) ? 0.0 : 1.0 / TotalWeight.Z),

	};
	OutTargetOffset *= InvWeight;
	
#if WITH_EDITOR
	if (Settings.bDebugDraw)
	{
		DebugInfo.TargetGoals.Last().Goal = OutTargetOffset + BoneLoc;
	}
#endif // WITH_EDITOR
	
	return TotalWeight;
}


void FIKRetargetRelativeIKOp::ComputeWeightMatrixRow(
	Eigen::MatrixXd& WeightMatrix,
	int32 RowIdx,
	const TArray<FName>& OutBones,
	const TArray<int32>& EffectIndices)
{
	for (int Index : EffectIndices)
	{
		const FRelativeIKFrameBoneConstraint& BoneConstraint = FrameBoneConstraints[Index];
		if (FMath::IsNearlyZero(BoneConstraint.Weight) || BoneConstraint.ConstraintType != ERelativeIKFrameBoneConstraintType::XYZ)
		{
			continue;
		}

		FName EffectBone = CachePairEffectBoneNames[Index];
		int32 OutWeightIdx = OutBones.Find(EffectBone);
		if (OutWeightIdx == INDEX_NONE)
		{
			continue;
		}

		WeightMatrix(RowIdx, OutWeightIdx) = -BoneConstraint.Weight;
	}
}

FVector FIKRetargetRelativeIKOp::ComputeTargetWeightedOffsetRow(
	Eigen::MatrixXd& OffsetMatrix,
	int32 RowIdx,
	const TArray<FName>& OutBones,
	const TArray<int32>& EffectIndices,
	const FVector& BoneLoc,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	FVector TotalWeight = FVector::ZeroVector;
	FVector OffsetLoc = FVector::ZeroVector;

#if WITH_EDITOR
	FVector PresolveTarget = FVector::ZeroVector;
	
	if (Settings.bDebugDraw)
	{
		DebugInfo.TargetGoals.AddDefaulted();
		DebugInfo.TargetGoals.Last().Goal = FVector::ZeroVector;
	}
#endif //WITH_EDITOR
	
	for (int Index : EffectIndices)
	{
		const FRelativeIKFrameBoneConstraint& BoneConstraint = FrameBoneConstraints[Index];
		if (FMath::IsNearlyZero(BoneConstraint.Weight))
		{
			continue;
		}

		FVector TargetOffset = BoneConstraint.Offset;
		FVector TargetWeight = FVector(BoneConstraint.Weight);
		if (BoneConstraint.ConstraintType == ERelativeIKFrameBoneConstraintType::Z)
		{
			TargetWeight = {0,0,BoneConstraint.Weight};
		}

		OffsetLoc += FrameBoneConstraints[Index].Weight * TargetOffset;
		TotalWeight += TargetWeight;

#if WITH_EDITOR
		if (Settings.bDebugDraw)
		{
			// TODO: Fixup target debug to show dynamic offset goals (post-solve update?)
			FVector PresolveBoneTarget = TargetOffset + BoneLoc;
			PresolveTarget += BoneConstraint.Weight * PresolveBoneTarget;
			
			DebugInfo.TargetGoals.Last().PairTargets.Add(PresolveBoneTarget);
			DebugInfo.PairRetargetInfo.Emplace(DebugPairSpaces[Index]);
		}
#endif //WITH_EDITOR
	}

	// Each matrix column is a dimension (x,y,z)
	OffsetMatrix(RowIdx,0) = OffsetLoc.X;
	OffsetMatrix(RowIdx,1) = OffsetLoc.Y;
	OffsetMatrix(RowIdx,2) = OffsetLoc.Z;
	
#if WITH_EDITOR
	if (Settings.bDebugDraw)
	{
		FVector InvWeight = {
			(FMath::IsNearlyZero(TotalWeight.X) ? 0.0 : 1.0 / TotalWeight.X),
			(FMath::IsNearlyZero(TotalWeight.Y) ? 0.0 : 1.0 / TotalWeight.Y),
			(FMath::IsNearlyZero(TotalWeight.Z) ? 0.0 : 1.0 / TotalWeight.Z),

		};
		DebugInfo.TargetGoals.Last().Goal = PresolveTarget * InvWeight;
	}
#endif // WITH_EDITOR
	
	return TotalWeight;
}



FName FIKRetargetRelativeIKOp::ApplyBodyMap(FName BodyName) const
{
	if (Settings.BodyMapping.IsEmpty() || !Settings.BodyMapping.Contains(BodyName))
	{
		return BodyName;
	}

	return Settings.BodyMapping[BodyName];
}

bool FIKRetargetRelativeIKOp::HasBoneDelta(FName TargetBoneName) const
{
	return CacheMapBoneSourceTargetTfm.Contains(TargetBoneName);
}

const FTransform& FIKRetargetRelativeIKOp::GetTargetBoneDelta(FName TargetBoneName) const
{
	if (!CacheMapBoneSourceTargetTfm.Contains(TargetBoneName))
	{
		return FTransform::Identity;
	}
	return CacheMapBoneSourceTargetTfm[TargetBoneName];
}

const FTransform& FIKRetargetRelativeIKOp::OffsetTransformsForSourceBones(FName SourceBoneName) const
{
	if (CachedNotifyInfo && CachedNotifyInfo->OffsetTransformsForBones.Contains(SourceBoneName))
	{
		return CachedNotifyInfo->OffsetTransformsForBones[SourceBoneName];
	}

	if (CachedPropNotifyInfo && CachedPropNotifyInfo->OffsetTransformsForBones.Contains(SourceBoneName))
	{
		return CachedPropNotifyInfo->OffsetTransformsForBones[SourceBoneName];
	}
	
	return FTransform::Identity;
}

void FIKRetargetRelativeIKOp::ComputeSourceBodyTransform(FBodyTransform& OutTransform, FName SourceBoneName, const FTransform& GlobalTfm, double SourceScale) const
{
	FTransform BoneOffsetTfm = OffsetTransformsForSourceBones(SourceBoneName);
	FQuat BodyRot = BoneOffsetTfm.GetRotation();
	FVector BodyTrans = BoneOffsetTfm.GetTranslation() * SourceScale;
	FVector BodyScale = BoneOffsetTfm.GetScale3D() * SourceScale;

	OutTransform.BoneToBody = FTransform(BodyRot.Inverse());
	OutTransform.BodyScale = BodyScale;
	OutTransform.BodyToGlobal = FTransform(BodyRot, BodyTrans) * GlobalTfm;
}

void FIKRetargetRelativeIKOp::ComputeTargetBodyTransform(FBodyTransform& OutTransform, FName SourceBoneName, const FTransform& GlobalTfm) const
{
	FName TargetBoneName = ApplyBodyMap(SourceBoneName);
	
	FQuat BodyRot = GetBodyRotation(TargetRelativeIKPhysicsAsset, TargetBoneName);
	FVector BodyTrans = GetBodyTranslation(TargetRelativeIKPhysicsAsset, TargetBoneName);
	// TODO: Probably should always use target body scale
	FVector BodyScale = GetBodyOrientedScale(TargetRelativeIKPhysicsAsset, TargetBoneName);
	
	OutTransform.BoneToBody = GetTargetBoneDelta(TargetBoneName) * FTransform(BodyRot.Inverse());
	OutTransform.BodyScale = BodyScale;
	OutTransform.BodyToGlobal = FTransform(BodyRot, BodyTrans) * GlobalTfm;
}

FVector FIKRetargetRelativeIKOp::ApplyBodyTransform(const FBodyTransform& Transform, const FVector& LocalPos)
{
	const FTransform& BodyLocalTfm = Transform.BoneToBody;
	const FVector& LocalScale = Transform.BodyScale;
	const FTransform& GlobalTfm = Transform.BodyToGlobal;
	
	return GlobalTfm.TransformPosition(BodyLocalTfm.TransformPositionNoScale(LocalPos) * LocalScale);
}

FVector FIKRetargetRelativeIKOp::InverseBodyTransform(const FBodyTransform& Transform, const FVector& GlobalPos)
{
	const FTransform& BodyLocalTfm = Transform.BoneToBody;
	const FVector& LocalScale = Transform.BodyScale;
	const FTransform& GlobalTfm = Transform.BodyToGlobal;

	return BodyLocalTfm.InverseTransformPositionNoScale(GlobalTfm.InverseTransformPosition(GlobalPos) / LocalScale);
}


void FIKRetargetRelativeIKOp::UpdateCacheChainInfo(const TArray<FTransform>& TargetPose, FName BoneName)
{
	if (!CacheBoneChains.Contains(BoneName))
	{
		return;
	}

	if (CacheChainLengthMap.Contains(BoneName))
	{
		return;
	}

	TArray<FTransform> ChainTransforms;
	ChainTransforms.Reserve(CacheBoneChains[BoneName]->BoneIndices.Num());
	for (int32 BoneIdx : CacheBoneChains[BoneName]->BoneIndices)
	{
		ChainTransforms.Add(TargetPose[BoneIdx]);
	}

	CacheChainLengthMap.Emplace(BoneName, CacheBoneChains[BoneName]->GetChainLength(ChainTransforms));
	CacheChainStartMap.Emplace(BoneName, TargetPose[CacheBoneChains[BoneName]->BoneIndices[0]].GetTranslation());
}

TConstArrayView<FVector3f> FIKRetargetRelativeIKOp::ApplyTemporalSmoothing(float Time, float SampleRate, int32 NumSamples, int32 NumBodies, const TArray<FVector3f>& PairLocalVerts)
{
	if (NumSamples <= 0)
	{
		return {};
	}
	
	int32 SampleIdx = FMath::Clamp(static_cast<int32>(Time * SampleRate), 0, NumSamples-1);

	// Testing temporal smoothing of local points
	const int32 MinSmoothIdx = FMath::Max(SampleIdx - Settings.TemporalSmoothingRadius, 0);
	const int32 MaxSmoothIdx = FMath::Min(SampleIdx + Settings.TemporalSmoothingRadius,NumSamples-1) ;
	
	if (Settings.TemporalSmoothingRadius <= 0)
	{
		int32 SampleIdx2 = FMath::Clamp(SampleIdx+1, 0, NumSamples-1);
		if (SampleIdx == SampleIdx2)
		{
			return {PairLocalVerts.GetData() + NumBodies * SampleIdx, NumBodies};
		}
		
		float lambda = static_cast<float>(SampleIdx2) - Time * SampleRate;
		SmoothedPoints.SetNumUninitialized(NumBodies);
		for (int BodyIdx = 0; BodyIdx < NumBodies; ++BodyIdx)
		{
			SmoothedPoints[BodyIdx] = lambda * PairLocalVerts[NumBodies * SampleIdx + BodyIdx] + (1.f-lambda) * PairLocalVerts[NumBodies * SampleIdx2 + BodyIdx];
		}
		return SmoothedPoints;
	}
	
	SmoothedPoints.SetNumUninitialized(NumBodies);
	for (int BodyIdx = 0; BodyIdx < NumBodies; ++BodyIdx)
	{
		SmoothedPoints[BodyIdx] = FVector3f::Zero();
		float SumWeight = 0.0f;
		for (int32 FrameIdx = MinSmoothIdx; FrameIdx <= MaxSmoothIdx; ++FrameIdx)
		{
			// Finite support Gaussian smoothing (could approximate w/ triangle or bake out smoothed data)
			const float Sigma = 0.25f * static_cast<float>(Settings.TemporalSmoothingRadius);
			const float SmoothDist = static_cast<float>(FrameIdx) - Time * SampleRate;
			const float SmoothWeight = FMath::Exp(-0.5f*(SmoothDist*SmoothDist) / (Sigma*Sigma));
			SmoothedPoints[BodyIdx] += SmoothWeight * PairLocalVerts[NumBodies * FrameIdx + BodyIdx];
			SumWeight += SmoothWeight;
		}
		SmoothedPoints[BodyIdx] /= SumWeight;
	}
	return SmoothedPoints;
}

void FIKRetargetRelativeIKOp::ComputeFeasibilityRange(FDoubleInterval& OutFeasibleRange, FName BoneName, const FVector& BoneLoc, const FVector& TargetOffsetVec)
{
	OutFeasibleRange = {0.0, 1.0};
	if (!CacheChainLengthMap.Contains(BoneName))
	{
		return;
	}
	
	FVector ChainRootLoc = CacheChainStartMap[BoneName];
	double CheckRadius = FMath::Max(CacheChainLengthMap[BoneName] + Settings.FeasibilityLengthBias, 0.0);

	FVector StartBoneLoc = BoneLoc;
	FVector EndBoneLoc = BoneLoc + TargetOffsetVec;
			
	double ChainDistStart = FVector::Distance(StartBoneLoc, ChainRootLoc);
	double ChainDistEnd = FVector::Distance(EndBoneLoc, ChainRootLoc);

	FDoubleInterval IntersectRange;
	if (FMath::Max(ChainDistStart, ChainDistEnd) <= CheckRadius)
	{
		// Whole range is feasible
		OutFeasibleRange = {0.0,1.0};
		return;
	}

	// If valid intersect: Sphere intersection range (return clamped to line segment) 
	// No valid intersect: Closest point to sphere (return clamped to line segment)
	CalcLineSphereIntersect(IntersectRange, ChainRootLoc, CheckRadius, StartBoneLoc, EndBoneLoc);
	OutFeasibleRange = {FMath::Clamp(IntersectRange.Min, 0.0,1.0), FMath::Clamp(IntersectRange.Max, 0.0,1.0)};
}

bool FIKRetargetRelativeIKOp::CalcLineSphereIntersect(FDoubleInterval& OutRange, const FVector& Center, double Radius, const FVector& StartPoint, const FVector& EndPoint)
{
	// NOTE: This function assumes two intersection points and returns valid positive ratios along line from start point, false if invalid/no intersection
	// Used for getting the endpoints of feasible range
	FVector LineVec = EndPoint - StartPoint;
	FVector LocPt = StartPoint - Center;

	double a = LineVec.SquaredLength();
	double b = FVector::DotProduct(LineVec, LocPt);
	double c = LocPt.SquaredLength() - Radius*Radius;
	if (LineVec.IsNearlyZero())
	{
		// Line is ill-defined, use midpoint
		OutRange = {0.5,0.5};
		return false;
	}
	
	if (b*b - a*c < 0.0)
	{
		// No intersection, return closest point on line (clamped to line segment outside this function)
		OutRange = {-b/a, -b/a};
		return false;
	}
	
	double B = FMath::Sqrt(b*b - a*c);
	double MaxRange = (-b + B) / a;
	double MinRange = (-b - B) / a;
	OutRange = {MinRange, MaxRange};

	return true;
}

FVector FIKRetargetRelativeIKOp::CalcReferenceShapeScale3D(FKShapeElem* ShapeElem)
{
	FVector ElementScale3D = FVector::OneVector;
	switch (ShapeElem->GetShapeType())
	{
	case EAggCollisionShape::Sphere:
		{
			FKSphereElem* SphereElem = static_cast<FKSphereElem*>(ShapeElem);
			ElementScale3D = FVector(SphereElem->Radius);
			break;
		}

	case EAggCollisionShape::Box:
		{
			FKBoxElem* BoxElem = static_cast<FKBoxElem*>(ShapeElem);
			ElementScale3D = FVector(BoxElem->X, BoxElem->Y, BoxElem->Z);
			break;
		}

	case EAggCollisionShape::Sphyl:
		{
			FKSphylElem* CapsuleElem = static_cast<FKSphylElem*>(ShapeElem);
			ElementScale3D = FVector(CapsuleElem->Radius,CapsuleElem->Radius,CapsuleElem->Length * 0.5f + CapsuleElem->Radius);
			break;
		}

	case EAggCollisionShape::Convex:
		{
			// UE_LOGF(LogAnimation, Warning, "Unsupported: Shape is a Convex");
			break;
		}
	default:
		{
			// UE_LOGF(LogAnimation, Warning, "Unknown or unsupported shape type");
			break;
		}
	}
	
	ensure(!ElementScale3D.IsNearlyZero());
	
	return ElementScale3D;
}

FQuat FIKRetargetRelativeIKOp::GetBodyRotation(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	FKShapeElem* ShapeElem = FindBodyShape(PhysAsset, BoneName);
	if (!ShapeElem)
	{
		return FQuat::Identity;
	}
	
	FTransform BodyTransform = ShapeElem->GetTransform();
	return BodyTransform.GetRotation();
}

FVector FIKRetargetRelativeIKOp::GetBodyTranslation(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	FKShapeElem* ShapeElem = FindBodyShape(PhysAsset, BoneName);
	if (!ShapeElem)
	{
		return FVector::ZeroVector;
	}
	
	FTransform BodyTransform = ShapeElem->GetTransform();
	return BodyTransform.GetTranslation();
}

FVector FIKRetargetRelativeIKOp::GetBodyOrientedScale(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	FKShapeElem* ShapeElem = FindBodyShape(PhysAsset, BoneName);
	if (!ShapeElem)
	{
		return FVector::OneVector;
	}
	
	return CalcReferenceShapeScale3D(ShapeElem);
}

FVector FIKRetargetRelativeIKOp::GoalLocBlendCompSpace(const FIKRigGoal* Goal, const FTransform& BoneTfm) const
{
	if (!Goal->bEnabled)
	{
		return BoneTfm.GetLocation();
	}

	switch (Goal->PositionSpace)
	{
		case EIKRigGoalSpace::Additive:
		{
			return FMath::Lerp(FVector::ZeroVector, Goal->Position, Goal->PositionAlpha) + BoneTfm.GetLocation();
		}
		case EIKRigGoalSpace::Component:
		{
			return FMath::Lerp(BoneTfm.GetLocation(), Goal->Position, Goal->PositionAlpha);
		}
		case EIKRigGoalSpace::World:
		default:
		{
			// We assume no World-space goals will be set using retarget stack
		}
	}

	return BoneTfm.GetLocation();
}

FKShapeElem* FIKRetargetRelativeIKOp::FindBodyShape(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	int32 BodyIdx = PhysAsset->FindBodyIndex(BoneName);
	if (BodyIdx == INDEX_NONE)
	{
		return nullptr;
	}
	return PhysAsset->SkeletalBodySetups[BodyIdx]->AggGeom.GetElement(0);
}

double FIKRetargetRelativeIKOp::GetDistanceWeight(EDistanceWeightMode DistanceWeightMode, double Distance, double DistThreshold, double DistHalfPointLambda)
{
	switch (DistanceWeightMode)
	{
	case EDistanceWeightMode::Linear:
		return RationalSmoothStepdown<1>(Distance, DistThreshold, DistHalfPointLambda);
	case EDistanceWeightMode::Quadratic:
		return RationalSmoothStepdown<2>(Distance, DistThreshold, DistHalfPointLambda);
	case EDistanceWeightMode::Cubic:
		return RationalSmoothStepdown<3>(Distance, DistThreshold, DistHalfPointLambda);
	case EDistanceWeightMode::Quartic:
		return RationalSmoothStepdown<4>(Distance, DistThreshold, DistHalfPointLambda);
	default: 
		if (Distance > DistThreshold)
		{
			return 0.;
		}
		return 1.;
	}
	
}

void FIKRetargetRelativeIKOp::AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent)
{
	// TODO: See if we should disable preupdate calls at the processor level?
	if (!IsEnabled() || !bIsInitialized)
	{
		return;
	}
	
	const FAnimMontageInstance* PrevMontageInstance = MontageInstance;
	MontageInstance = nullptr;

	UAnimInstance* SourceAnimInstance = SourceMeshComponent.GetAnimInstance();
	if (!SourceAnimInstance)
	{
		return;
	}
	
	MontageInstance = SourceAnimInstance->GetActiveMontageInstance();
	if (MontageInstance)
	{
		// Early-out for montages
		if (MontageInstance == PrevMontageInstance)
		{
			PreUpdateMontagePlayhead();
			return;
		}
		
		SetupRelativeIKNotifyInfoMontage();
		return;
	}

	UpdateRelativeIKNotifyInfoAnimSeq(SourceAnimInstance);
}

void FIKRetargetRelativeIKOp::UpdateCacheBoneChains(const FIKRetargetProcessor& InProcessor, const FIKRetargetRunIKRigOp* ParentRigOp)
{
	CacheBoneChains.Reset();

	// TODO: This is a reinit case from parent (need to handle appropriately)
	// Cache resolved chains in target map by goal bone
	TArray<FName> RequiredChains = ParentRigOp->GetRequiredTargetChains();
	const FRetargeterBoneChains& RetargetBoneChains = InProcessor.GetBoneChains();
	
	for (FName Chain : RequiredChains)
	{
		const FResolvedBoneChain* BoneChain = RetargetBoneChains.GetResolvedBoneChainByName(Chain, ERetargetSourceOrTarget::Target, ParentRigOp->Settings.IKRigAsset);
		if (!BoneChain)
		{
			continue;
		}

		CacheBoneChains.Emplace(BoneChain->EndBone, BoneChain);
	}
}

void FIKRetargetRelativeIKOp::UpdateSourceBoneMap(FName SourceBoneName)
{
	int32 SourceIdx = SourceBoneNames.Find(SourceBoneName);
	if (SourceIdx == INDEX_NONE)
	{
		CacheSourceSkelIndices.Remove(SourceBoneName);
		return;
	}

	CacheSourceSkelIndices.Emplace(SourceBoneName, SourceIdx);
}

void FIKRetargetRelativeIKOp::UpdateTargetBoneMap(FName TargetBoneName)
{
	int32 TargetIdx = TargetBoneNames.Find(TargetBoneName);
	if (TargetIdx == INDEX_NONE)
	{
		CacheTargetSkelIndices.Remove(TargetBoneName);
		return;
	}
	
	CacheTargetSkelIndices.Emplace(TargetBoneName, TargetIdx);
}

void FIKRetargetRelativeIKOp::UpdateBoneMapTfm(FName SourceBoneName, FName TargetBoneName)
{
	int32 SourceIdx = SourceBoneNames.Find(SourceBoneName);
	int32 TargetIdx = TargetBoneNames.Find(TargetBoneName);
	if (SourceIdx == INDEX_NONE || TargetIdx == INDEX_NONE)
	{
		CacheMapBoneSourceTargetTfm.Remove(TargetBoneName);
		return;
	}

	FQuat SourceRot = CacheSourceBoneInitTfm[SourceIdx].GetRotation();
	FQuat TargetInvRot = CacheTargetBoneInitTfm[TargetIdx].GetRotation().Inverse();

	FTransform TargetDelta = FTransform(TargetInvRot*SourceRot);
	CacheMapBoneSourceTargetTfm.Emplace(TargetBoneName,TargetDelta);
}

void FIKRetargetRelativeIKOp::UpdateCacheSkelInfo(const FRetargetSkeleton& InSourceSkeleton, const FTargetSkeleton& InTargetSkeleton)
{
	SourceBoneNames = InSourceSkeleton.BoneNames;
	TargetBoneNames = InTargetSkeleton.BoneNames;

	CacheSourceBoneInitTfm = InSourceSkeleton.RetargetPoses.GetGlobalRetargetPose();
	CacheTargetBoneInitTfm = InTargetSkeleton.RetargetPoses.GetGlobalRetargetPose();

	// TODO: Cache less data/directly bake source bone indices and use retarget chains instead of bone mapping
	CacheSourceSkelIndices.Reset();
	CacheTargetSkelIndices.Reset();
}

void FIKRetargetRelativeIKOp::PreUpdateMontagePlayhead()
{
	if (!MontageInstance)
	{
		UpdateAnimPlayhead(-1.0f);
		return;
	}
	
	float MontageTime = MontageInstance->GetPosition();
	if (MontageTime < SegmentStartTime || MontageTime > SegmentEndTime)
	{
		UpdateAnimPlayhead(-1.0f);
		return;
	}
	
	UpdateAnimPlayhead(MontageTime - SegmentStartTime);
}

void FIKRetargetRelativeIKOp::SetupRelativeIKNotifyInfoMontage()
{
	ResetCacheNotifyInfo();
	
	CacheSourceAnimSequence = nullptr;
	UAnimMontage* Montage = MontageInstance->Montage;
	if (!Montage)
	{
		return;
	}
	
	for (const FSlotAnimationTrack& SlotTrack : Montage->SlotAnimTracks)
	{
		for (const FAnimSegment& Segment : SlotTrack.AnimTrack.AnimSegments)
		{
			const TObjectPtr<UAnimSequenceBase>& SeqRef = Segment.GetAnimReference();
			UAnimSequence* AnimSeq = Cast<UAnimSequence>(SeqRef.Get());
			if (!AnimSeq)
			{
				continue;
			}

			// TODO: This loop only really works w/ at most one of each notify
			for (const FAnimNotifyEvent& NotifyEvent : SeqRef->Notifies)
			{
				const TObjectPtr<UAnimNotify>& Notify = NotifyEvent.Notify;
				if (Cast<URelativeBodyBakeAnimNotify>(Notify) || Cast<URelativePropsBakeAnimNotify>(Notify))
				{
					UpdateAnimSeqData(AnimSeq);
					
					// Have to cache these for quick updating montage frame-time
					SegmentStartTime = Segment.AnimStartTime;
					SegmentEndTime = Segment.AnimEndTime;
					PreUpdateMontagePlayhead();
					return;
				}
			}
		}
	}
}

void FIKRetargetRelativeIKOp::UpdateRelativeIKNotifyInfoAnimSeq(UAnimInstance* SourceAnimInstance)
{
	// Get source tick records
	TMap<FName, FAnimGroupInstance> SyncGroupTickRecords = SourceAnimInstance->GetSyncGroupMapRead();
	TArray<FAnimTickRecord> UngroupedTickRecords = SourceAnimInstance->GetUngroupedActivePlayersRead();

	// Check for active players with baked notify data, keep hightest blend amount
	float MaxBlend = -1.0f;
	const FAnimTickRecord* MaxNotifyRecord = nullptr;
	auto ProcessTickRecords = [&MaxBlend,&MaxNotifyRecord](const TArray<FAnimTickRecord>& ActivePlayers)
		{
			for (const FAnimTickRecord& ActivePlayer : ActivePlayers)
			{
				if ( ActivePlayer.EffectiveBlendWeight <= MaxBlend )
				{
					continue;
				}
				if (UAnimSequence* Sequence = Cast<UAnimSequence>(ActivePlayer.SourceAsset))
				{
					// Used to determine which notifies should we considered "active".
					check(ActivePlayer.DeltaTimeRecord->IsPreviousValid())
					for (const FAnimNotifyEvent& NotifyEvent : Sequence->Notifies)
					{
						if (URelativeBodyBakeAnimNotify* BakedNotifyInfo = Cast<URelativeBodyBakeAnimNotify>(NotifyEvent.Notify))
						{
							MaxBlend = ActivePlayer.EffectiveBlendWeight;
							MaxNotifyRecord = &ActivePlayer;
						}
						else if (URelativePropsBakeAnimNotify* PropInfo = Cast<URelativePropsBakeAnimNotify>(NotifyEvent.Notify))
						{
							MaxBlend = ActivePlayer.EffectiveBlendWeight;
							MaxNotifyRecord = &ActivePlayer;
						}
					}
				}
			}
		};
	
	// Find all the anim sequences (with sync groups) we updated this tick.
	for (const TTuple<FName, FAnimGroupInstance>& SyncGroupPair : SyncGroupTickRecords)
	{
		ProcessTickRecords(SyncGroupPair.Value.ActivePlayers);
	}
	
	// Find all the anim sequences (no sync groups) we updated this tick.
	ProcessTickRecords(UngroupedTickRecords);
	
	float FrameTime = 0.0f;
	UAnimSequence* AnimSeq = nullptr;
	if (MaxNotifyRecord)
	{
		FrameTime = MaxNotifyRecord->DeltaTimeRecord->GetPrevious() + MaxNotifyRecord->DeltaTimeRecord->Delta;
		AnimSeq = Cast<UAnimSequence>(MaxNotifyRecord->SourceAsset);
	}

	UpdateAnimPlayhead(FrameTime);
	UpdateAnimSeqData(AnimSeq);
}

void FIKRetargetRelativeIKOp::UpdateAnimPlayhead(float FrameTime)
{
	AnimSeqPlayHead = FrameTime;
}

void FIKRetargetRelativeIKOp::UpdateAnimSeqData(UAnimSequence* Sequence)
{
	if (Sequence == CacheSourceAnimSequence)
	{
		return;
	}

	ResetCacheNotifyInfo();
	CacheSourceAnimSequence = Sequence;
	
	if (!Sequence)
	{
		return;
	}

	// TODO: This loop only really works w/ at most one of each notify
	for (const FAnimNotifyEvent& NotifyEvent : CacheSourceAnimSequence->Notifies)
	{
		if (URelativeBodyBakeAnimNotify* BakedNotifyInfo = Cast<URelativeBodyBakeAnimNotify>(NotifyEvent.Notify))
		{
			UpdateCacheNotifyInfo(BakedNotifyInfo);
		}
		else if (URelativePropsBakeAnimNotify* PropInfo = Cast<URelativePropsBakeAnimNotify>(NotifyEvent.Notify))
		{
			UpdateCachePropNotifyInfo(PropInfo);
		}
	}
}

void FIKRetargetRelativeIKOp::ResetCacheNotifyInfo()
{
	SegmentStartTime = -1.0f;
	SegmentEndTime = -1.0f;
	CachedNotifyInfo = nullptr;
	CachedPropNotifyInfo = nullptr;
	CacheMeshPairOffset = 0;
	CachePropPairOffset = 0;

	// TODO: Need to bake info in accessible bipartite graph structure and cache less data!
	// Clear all cached data on update
	CacheMapBoneSourceTargetTfm.Reset();
	CacheSourceBodyEffectVertIdx.Reset();
	CacheSourceEffectBones.Reset();
	
	CacheBodyBones.Reset();
	CachePairEffectBoneNames.Reset();
	FrameBoneConstraints.Reset();
	
	CachePinSourcePropBones.Reset();
}

void FIKRetargetRelativeIKOp::UpdateCacheNotifyInfo(URelativeBodyBakeAnimNotify* NotifyInfo)
{
	if (NotifyInfo->OffsetTransformsForBones.IsEmpty())
	{
		return;
	}

	if (!CachedNotifyInfo)
	{
		CachedNotifyInfo = NotifyInfo;
	}
	else
	{
		return;
	}
	CacheMeshPairOffset = FrameBoneConstraints.Num() / 2;
	
	int32 NumBodies = CachedNotifyInfo->BodyPairs.Num();
	FrameBoneConstraints.AddDefaulted(NumBodies);
	CachePairEffectBoneNames.AddDefaulted(NumBodies);

	MeshSampleRate = 30.0f;
	if (CachedNotifyInfo->BodyPairsSampleTime.Num() >= 2)
	{
		const float DeltaT = CachedNotifyInfo->BodyPairsSampleTime[1] - CachedNotifyInfo->BodyPairsSampleTime[0];
		MeshSampleRate = FMath::RoundToFloat(1.0f / DeltaT);
	}
	
	for (const TPair<FName, FTransform>& BodyTfmPair : CachedNotifyInfo->OffsetTransformsForBones)
	{
		FName SourceBodyBone = BodyTfmPair.Key;
		if (CacheSourceEffectBones.Contains(SourceBodyBone))
		{
			continue;
		}
		
		CacheSourceEffectBones.Add(SourceBodyBone);
		CacheBodyBones.Add(SourceBodyBone);
		CacheSourceBodyEffectVertIdx.AddDefaulted();

		FName TargetBodyBone = ApplyBodyMap(SourceBodyBone);
		UpdateSourceBoneMap(SourceBodyBone);
		UpdateTargetBoneMap(TargetBodyBone);
		UpdateBoneMapTfm(SourceBodyBone, TargetBodyBone);
	}

	// Cache bone-bone effectors for creating weighted goals
	for (int PairIdx = 0; PairIdx < CachedNotifyInfo->bBodyPairsIsParentDominates.Num(); ++PairIdx)
	{
		int32 OutPairIdx = PairIdx + CacheMeshPairOffset;
		bool bPrimaryFixed = CachedNotifyInfo->bBodyPairsIsParentDominates[PairIdx];
		
		FName PrimaryBone = CachedNotifyInfo->BodyPairs[2*PairIdx];
		FName OtherBone = CachedNotifyInfo->BodyPairs[2*PairIdx + 1];

		int32 PrimaryIdx = CacheSourceEffectBones.Find(PrimaryBone);
		int32 OtherIdx = CacheSourceEffectBones.Find(OtherBone);

		// Secondary (contact) Bone always affected
		CacheSourceBodyEffectVertIdx[OtherIdx].Add(2*OutPairIdx+1);
		if (!bPrimaryFixed)
		{
			// Primary bone can be affect if e.g. hands clapping
			CacheSourceBodyEffectVertIdx[PrimaryIdx].Add(2*OutPairIdx);
		}
		
		// TODO: Normalize pair edge info/frame info
		CachePairEffectBoneNames[2*OutPairIdx] = OtherBone;
		CachePairEffectBoneNames[2*OutPairIdx + 1] = PrimaryBone;
	}
}

void FIKRetargetRelativeIKOp::UpdateCachePropNotifyInfo(URelativePropsBakeAnimNotify* NotifyInfo)
{
	if (NotifyInfo->bPropsPairsIsParentDominates.IsEmpty())
	{
		return;
	}
	
	if (!CachedPropNotifyInfo)
	{
		CachedPropNotifyInfo = NotifyInfo;
	}
	else
	{
		return;
	}
	CachePropPairOffset = FrameBoneConstraints.Num() / 2;

	int32 NumBodies = CachedPropNotifyInfo->PropsPairs.Num();
	FrameBoneConstraints.AddDefaulted(NumBodies);
	CachePairEffectBoneNames.AddDefaulted(NumBodies);

	PropSampleRate = 30.0f;
	if (CachedPropNotifyInfo->PropsPairsSampleTime.Num() >= 2)
	{
		const float DeltaT = CachedPropNotifyInfo->PropsPairsSampleTime[1] - CachedPropNotifyInfo->PropsPairsSampleTime[0];
		PropSampleRate = FMath::RoundToFloat(1.0f / DeltaT);
	}
	
	for (const TPair<FName, FTransform>& BodyTfmPair : CachedPropNotifyInfo->OffsetTransformsForBones)
	{
		FName SourceBodyBone = BodyTfmPair.Key;
		if (CacheSourceEffectBones.Contains(SourceBodyBone))
		{
			continue;
		}
		
		CacheSourceEffectBones.Add(SourceBodyBone);
		CacheSourceBodyEffectVertIdx.AddDefaulted();

		FName TargetBodyBone = ApplyBodyMap(SourceBodyBone);
		UpdateSourceBoneMap(SourceBodyBone);
		UpdateTargetBoneMap(TargetBodyBone);
		UpdateBoneMapTfm(SourceBodyBone, TargetBodyBone);
	}

	// Cache bone-bone effectors for creating weighted goals
	for (int PairIdx = 0; PairIdx < CachedPropNotifyInfo->bPropsPairsIsParentDominates.Num(); ++PairIdx)
	{
		int32 OutPairIdx = PairIdx + CachePropPairOffset;
		bool bBodyFixed = CachedPropNotifyInfo->bPropsPairsIsParentDominates[PairIdx];
		
		FName PropBone = CachedPropNotifyInfo->PropsPairs[2*PairIdx];
		FName BodyBone = CachedPropNotifyInfo->PropsPairs[2*PairIdx + 1];

		int32 PropIdx = CacheSourceEffectBones.Find(PropBone);
		int32 BodyIdx = CacheSourceEffectBones.Find(BodyBone);

		// TODO: Normalize pair edge info for all bake types
		CachePairEffectBoneNames[2*OutPairIdx] = BodyBone;
		CachePairEffectBoneNames[2*OutPairIdx + 1] = PropBone;

		if (BodyBone == FName("Floor"))
		{
			// Prop attach bones always allowed to move
			CacheSourceBodyEffectVertIdx[PropIdx].Add(2*OutPairIdx);

			// Keep track of prop attach bones for post-solve pin/update
			CachePinSourcePropBones.AddUnique(PropBone);
			
			continue;
		}

		if (PropIdx == INDEX_NONE || BodyIdx == INDEX_NONE)
		{
			continue;
		}
		
		// Prop attach bones always allowed to move
		CacheSourceBodyEffectVertIdx[PropIdx].Add(2*OutPairIdx);
		if (!bBodyFixed)
		{
			// TODO: Only add IK goal bones here!
			CacheSourceBodyEffectVertIdx[BodyIdx].Add(2*OutPairIdx+1);
		}

		// Keep track of prop attach bones for post-solve pin/update
		CachePinSourcePropBones.AddUnique(PropBone);
	}
	
	CachePropId2PinTargetPropSettingId.Init(INDEX_NONE, CachePinSourcePropBones.Num());
	CachePropId2PinBoneDataId.Init(INDEX_NONE, CachePinSourcePropBones.Num());
	int32 PinBoneId = 0;
	for (int32 SettingId = 0; SettingId < Settings.PinPropBones.Num(); ++SettingId)
	{
		const FPinBoneSettings& PinBoneInfo = Settings.PinPropBones[SettingId];
		FName SourceBoneName = PinBoneInfo.SourceBoneName;
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		
		int32 CachePropIdx = CachePinSourcePropBones.Find(SourceBoneName);
		if (CachePropIdx != INDEX_NONE)
		{
			CachePropId2PinTargetPropSettingId[CachePropIdx] = SettingId;
		}
		
		if (TargetBoneName != NAME_None && CacheSourceSkelIndices.Contains(SourceBoneName) && CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			if (CachePropIdx != INDEX_NONE)
			{
				CachePropId2PinBoneDataId[CachePropIdx] = PinBoneId;
			}
			PinBoneId++;
		}
	}
}


void FIKRetargetRelativeIKOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
}

FIKRetargetOpSettingsBase* FIKRetargetRelativeIKOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetRelativeIKOp::GetSettingsType() const
{
	return FIKRetargetRelativeIKOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetRelativeIKOp::GetType() const
{
	return FIKRetargetRelativeIKOp::StaticStruct();
}

const UScriptStruct* FIKRetargetRelativeIKOp::GetParentOpType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

void FIKRetargetRelativeIKOp::OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent)
{
}

void FIKRetargetRelativeIKOp::GetAnimSeqFramePose(const UAnimSequence* AnimSeq, double Time, TArray<FName>& OutBones, TArray<FTransform>& OutPose) const
{
	const FReferenceSkeleton& Skel = AnimSeq->GetSkeleton()->GetReferenceSkeleton();
	int32 NumBones = Skel.GetNum();
	
	OutBones.Reset(NumBones);
	OutPose.Reset(NumBones);

	// Init bone container for pulling animseq data
	TArray<FBoneIndexType> BoneIndices;
	BoneIndices.Reserve(NumBones);
	for (FBoneIndexType BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		BoneIndices.Add(BoneIndex);
		OutBones.Add(Skel.GetBoneName(BoneIndex));
	}
	
	FBoneContainer BoneContainer;
	BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *AnimSeq->GetSkeleton());
	BoneContainer.SetUseRAWData(false);
	
	// Setup bone container
	FCompactPose CompactPose;
	FBlendedCurve Curves;
	UE::Anim::FStackAttributeContainer Attributes;
	FAnimationPoseData FramePoseData(CompactPose, Curves, Attributes);
	
	CompactPose.SetBoneContainer(&BoneContainer);
	Curves.InitFrom(BoneContainer);
	
	FAnimExtractContext FrameExtractionCtx(Time, false);
	AnimSeq->GetAnimationPose(FramePoseData, FrameExtractionCtx);
	
	FramePoseData.GetPose().CopyBonesTo(OutPose);
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const int32 ParentBoneIndex = Skel.GetParentIndex(BoneIndex);

		FTransform ParentTransform = FTransform::Identity;
		if (ParentBoneIndex != INDEX_NONE)
		{
			// TODO: Double-check parents always befor children in list
			FString ParentBoneName = Skel.GetBoneName(ParentBoneIndex).ToString();
			ParentTransform = OutPose[ParentBoneIndex];
		}
		
		OutPose[BoneIndex] = OutPose[BoneIndex] * ParentTransform;
	}
}

#if WITH_EDITOR

void FIKRetargetRelativeIKOp::UpdateFromAnimSequence(UAnimSequence* Sequence, float FrameTime)
{
	UpdateAnimPlayhead(FrameTime);
	UpdateAnimSeqData(Sequence);
}


FCriticalSection FIKRetargetRelativeIKOp::DebugDataMutex;

void FIKRetargetRelativeIKOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	if (!Settings.bDebugDraw)
	{
		return;
	}
	
	FScopeLock ScopeLock(&DebugDataMutex);
	if (Settings.bDebugDrawBodyPairs)
	{
		// Draw Source body pairs
		DebugDrawBodyPairs(InPDI, InSourceTransform, 1.0, DebugDrawInfo.SourcePairVerts);

		// Draw target body pairs (Possibly with "full" retarget space display)
		DebugDrawBodyPairs(InPDI, InComponentTransform, InComponentScale, DebugDrawInfo.TargetPairVerts);
	}
	
	if (Settings.bDebugDrawPhysicsBodies)
	{
		if (Settings.TargetPhysicsAssetOverride)
		{
			// TODO: if only bake source transforms instead of PA, this source draw is meaningless
			DebugDrawBodies(InPDI, InSourceTransform, 1.0, Settings.TargetPhysicsAssetOverride, DebugDrawInfo.SourceBodyInfo, true);
			DebugDrawBodies(InPDI, InComponentTransform, InComponentScale, Settings.TargetPhysicsAssetOverride, DebugDrawInfo.TargetBodyInfo, false);
		}
	}
	
	if (Settings.bDebugDrawBodyTransforms)
	{
		DebugDrawBodyCoords(InPDI, InSourceTransform, 1.0, DebugDrawInfo.SourceTfmInfo);
		DebugDrawBodyCoords(InPDI, InComponentTransform, InComponentScale,  DebugDrawInfo.TargetTfmInfo);
	}

	if (Settings.bDebugDrawRetargetVertAverages)
	{
		// Show retarget contribution average with offsets
		DebugDrawPairVertRetarget(InPDI, InComponentTransform, InComponentScale, DebugDrawInfo.PairRetargetInfo);
	}

	if (Settings.bDebugDrawGoalContributions)
	{
		// Show IK Goals (and contribs)
		DebugDrawGoalContributions(InPDI, InComponentTransform, InComponentScale, DebugDrawInfo.TargetGoals);
	}
}

void FIKRetargetRelativeIKOp::DebugDrawBodyPairs(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugDrawBodyPair>& BodyPairs) const
{
	const float MaxScale = 1.0f;
	const float MarkerSize = FMath::Min(MaxScale, static_cast<float>(Scale));
	for (const FDebugDrawBodyPair& Pair : BodyPairs)
	{
		float Scalar = FMath::Clamp((float)Pair.Weight, 0.f, 1.f);
		if (FMath::IsNearlyZero(Scalar))
		{
			continue;
		}
		FTransform TfmBodyA_A = FTransform(Pair.PosA_A) * BaseTransform;
		FTransform TfmBodyB_B = FTransform(Pair.PosB_B) * BaseTransform;

		IKRigDebugRendering::DrawWireCube(InPDI, TfmBodyA_A, FLinearColor::Blue*Scalar,MarkerSize,MarkerSize);
		IKRigDebugRendering::DrawWireCube(InPDI, TfmBodyB_B, FLinearColor::Red*Scalar,MarkerSize,MarkerSize);

		// Possibly draw full retarget debug info
		if (Pair.bFullPos)
		{
			FTransform TfmBodyA_B = FTransform(Pair.PosA_B) * BaseTransform;
			FTransform TfmBodyB_A = FTransform(Pair.PosB_A) * BaseTransform;

			IKRigDebugRendering::DrawWireCube(InPDI, TfmBodyA_B, FLinearColor::Black,MarkerSize,MarkerSize);
			IKRigDebugRendering::DrawWireCube(InPDI, TfmBodyB_A, FLinearColor::Yellow,MarkerSize,MarkerSize);

			// Draw direct relationships between bodies in retargeted A/B spaces
			DrawDashedLine(InPDI, TfmBodyA_A.GetLocation(), TfmBodyB_A.GetLocation(), FLinearColor::Black, 1, SDPG_Foreground);
			DrawDashedLine(InPDI, TfmBodyA_B.GetLocation(), TfmBodyB_B.GetLocation(), FLinearColor::Black, 1, SDPG_Foreground);

			// Yellow line showing difference between body A in A-retarget/B-retarget
			DrawDashedLine(InPDI, TfmBodyA_A.GetLocation(), TfmBodyA_B.GetLocation(), FLinearColor::Yellow, 1, SDPG_Foreground);
			// Red line showing difference between body B in A-retarget/B-retarget
			DrawDashedLine(InPDI, TfmBodyB_A.GetLocation(), TfmBodyB_B.GetLocation(), FLinearColor::Red, 1, SDPG_Foreground);
		}
		else
		{
			DrawDashedLine(InPDI, TfmBodyA_A.GetLocation(), TfmBodyB_B.GetLocation(), FLinearColor::Black, 1, SDPG_Foreground);
		}
	}
}

void FIKRetargetRelativeIKOp::DebugDrawGoalContributions(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugDrawTargetGoal>& GoalInfoLis) const
{
	// Show IK Goals (and contribs)
	const float MaxScale = 1.0f;
	const float MarkerSize = FMath::Min(MaxScale, static_cast<float>(Scale));
	for (const FDebugDrawTargetGoal& Goal : GoalInfoLis)
	{
		FTransform GoalTfm = FTransform(Goal.Goal) * BaseTransform;
		FVector GoalPoint = GoalTfm.GetTranslation();
		for (const FVector& v: Goal.PairTargets)
		{
			FTransform ContribTfm = FTransform(v) * BaseTransform;
			FVector ContribPt = ContribTfm.GetTranslation();
			DrawDashedLine(InPDI, ContribPt, GoalPoint, FLinearColor::Red, 1, SDPG_Foreground);
			IKRigDebugRendering::DrawWireCube(InPDI, ContribTfm, FLinearColor::White,MarkerSize,MarkerSize);
		}
		IKRigDebugRendering::DrawWireCube(InPDI, GoalTfm, FLinearColor::Yellow,MarkerSize,MarkerSize);
	}
}

void FIKRetargetRelativeIKOp::DebugDrawPairVertRetarget(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugRelativeTargetPairSpace>& RetargetPairList) const
{
	const float MaxScale = 1.0f;
	const float MarkerSize = 0.5f * FMath::Min(MaxScale, static_cast<float>(Scale));
	for (const FDebugRelativeTargetPairSpace& Info : RetargetPairList)
	{
		FTransform V1Tfm = FTransform(Info.PairRangeStart) * BaseTransform;
		FTransform V2Tfm = FTransform(Info.PairRangeEnd) * BaseTransform;
		IKRigDebugRendering::DrawWireCube(InPDI, V1Tfm, FLinearColor::Red,MarkerSize,MarkerSize);
		IKRigDebugRendering::DrawWireCube(InPDI, V2Tfm, FLinearColor::Red,MarkerSize,MarkerSize);

		FVector V1 = V1Tfm.GetTranslation();
		FVector V2 = V2Tfm.GetTranslation();
		// DrawDashedLine(InPDI, VBone, V1, FLinearColor::Yellow, 1, SDPG_Foreground);
		// DrawDashedLine(InPDI, VBone, V2, FLinearColor::Yellow, 1, SDPG_Foreground);
		DrawDashedLine(InPDI, V1, V2, FLinearColor::Black, 1, SDPG_Foreground);

		FVector PairTargetV = FMath::Lerp(Info.PairRangeStart, Info.PairRangeEnd, Info.TargetAlpha);
		FTransform VAvgTfm = FTransform(PairTargetV) * BaseTransform;
		IKRigDebugRendering::DrawWireCube(InPDI, VAvgTfm, FLinearColor::Gray,MarkerSize,MarkerSize);

		FVector FeasibleMinV =  FMath::Lerp(Info.PairRangeStart, Info.PairRangeEnd, Info.FeasibleRange.Min);
		FTransform VMinTfm = FTransform(FeasibleMinV) * BaseTransform;
		IKRigDebugRendering::DrawWireCube(InPDI, VMinTfm, FLinearColor::Yellow,MarkerSize,MarkerSize);

		FVector FeasibleMaxV =  FMath::Lerp(Info.PairRangeStart, Info.PairRangeEnd, Info.FeasibleRange.Max);
		FTransform VMaxTfm = FTransform(FeasibleMaxV) * BaseTransform;
		IKRigDebugRendering::DrawWireCube(InPDI, VMaxTfm, FLinearColor::Green,MarkerSize,MarkerSize);
		// DrawDashedLine(InPDI, VOffset, VOffset + t.Get<3>(), FLinearColor::Black, 1, SDPG_Foreground);
	}
}

void FIKRetargetRelativeIKOp::DebugDrawBodies(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, UPhysicsAsset* PhysAsset, const TArray<FDebugDrawBodyInfo>& PhysBodies, bool TargetPAForSource) const
{
	for (const FDebugDrawBodyInfo& BodyInfo : PhysBodies)
	{
		FName TargetBodyName = TargetPAForSource? ApplyBodyMap(BodyInfo.BodyName) : BodyInfo.BodyName;
		FKShapeElem* ShapeElem = FindBodyShape(PhysAsset, TargetBodyName);
		if (!ShapeElem)
		{
			continue;
		}
		
		FTransform CompTfm = BodyInfo.BoneTfm * BaseTransform;
		if (TargetPAForSource)
		{
			FTransform TargetShapeTfm = ShapeElem->GetTransform();
			FTransform SourceShapeTfm = BodyInfo.RetargetTfm;
			SourceShapeTfm.RemoveScaling();
			
			CompTfm = TargetShapeTfm.Inverse() * SourceShapeTfm * CompTfm;
			CompTfm.SetScale3D(BodyInfo.RetargetTfm.GetScale3D()/CalcReferenceShapeScale3D(ShapeElem));
		}

		// Draw physics body
		if (DebugDrawInfo.TargetDomainBones.Contains(TargetBodyName))
		{
			DebugDrawPhysBody(InPDI, CompTfm, Scale * BodyInfo.Scale, ShapeElem, FLinearColor::Black);
		}
		else
		{
			DebugDrawPhysBody(InPDI, CompTfm, Scale * BodyInfo.Scale, ShapeElem, FLinearColor::Gray);
		}
	}
}

void FIKRetargetRelativeIKOp::DebugDrawPhysBody(FPrimitiveDrawInterface* InPDI, const FTransform& ParentTransform, double Scale, const FKShapeElem* ShapeElem, const FLinearColor& Color) const
{
	FVector AbsBodyScale = (ParentTransform.GetScale3D() * Scale).GetAbs();
	FTransform ParentTransformNoScale(ParentTransform);
	ParentTransformNoScale.RemoveScaling();
	FTransform BodyFrame = ShapeElem->GetTransform() * ParentTransformNoScale;

	const FVector Translation = BodyFrame.GetLocation();
	const FVector UnitXAxis = BodyFrame.GetUnitAxis( EAxis::X );
	const FVector UnitYAxis = BodyFrame.GetUnitAxis( EAxis::Y );
	const FVector UnitZAxis = BodyFrame.GetUnitAxis( EAxis::Z );

	
	switch (ShapeElem->GetShapeType())
	{
	case EAggCollisionShape::Box:
		{
			const FKBoxElem* BoxElem = static_cast<const FKBoxElem*>(ShapeElem);
			const FVector Extent = 0.5 * FVector(BoxElem->X, BoxElem->Y, BoxElem->Z) * AbsBodyScale;
			DrawOrientedWireBox(InPDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, Extent, Color, SDPG_Foreground);
			return;
		}
	case EAggCollisionShape::Sphyl:
		{
			const FKSphylElem* CapsuleElem = static_cast<const FKSphylElem*>(ShapeElem);
			const double Radius =CapsuleElem->GetScaledRadius(AbsBodyScale);
			const double HalfHeight = CapsuleElem->GetScaledHalfLength(AbsBodyScale);
			DrawWireCapsule(InPDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, Color,
						Radius, HalfHeight,25, SDPG_Foreground, 0, 1.0);
			return;
		}
	case EAggCollisionShape::Sphere:
		{
			const FKSphereElem* SphereElem = static_cast<const FKSphereElem*>(ShapeElem);
			const double Radius = SphereElem->Radius * AbsBodyScale.GetMin();
			DrawWireSphere(InPDI, Translation, Color, Radius, 25, SDPG_Foreground);
			return;
		}
	default:
		return;
	}
}


void FIKRetargetRelativeIKOp::DebugDrawBodyCoords(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugBodyTfmInfo>& BodyTfms) const
{
	// Stop scale from making coords too large
	const float MaxScale = 1.0f;
	const float CoordSize = 10.0f * FMath::Min(MaxScale, static_cast<float>(Scale));
	const float CoordThickness = 0.5f * FMath::Min(MaxScale, static_cast<float>(Scale));

	for (const FDebugBodyTfmInfo& BodyTfm : BodyTfms)
	{

		FVector StartV = BaseTransform.TransformPosition(BodyTfm.Center);
		FVector CoordX = BaseTransform.TransformPosition(BodyTfm.TfmX);
		FVector CoordY = BaseTransform.TransformPosition(BodyTfm.TfmY);
		FVector CoordZ = BaseTransform.TransformPosition(BodyTfm.TfmZ);
		
		InPDI->DrawLine(StartV, CoordX, FLinearColor::Red, SDPG_Foreground, CoordThickness);
		InPDI->DrawLine(StartV, CoordY, FLinearColor::Green, SDPG_Foreground, CoordThickness);
		InPDI->DrawLine(StartV, CoordZ, FLinearColor::Blue, SDPG_Foreground, CoordThickness);
	}
}

void FIKRetargetRelativeIKOp::UpdateDebugInfo(
	double SourceScale,
	const TArray<FTransform>& SourcePose,
	const TArray<FTransform>& TargetPose,
	const FDebugRelativeIKDrawInfo& LocalDebugInfo)
{
	if (!Settings.bDebugDraw)
	{
		return;
	}
	
	FScopeLock ScopeLock(&DebugDataMutex);
	
	DebugDrawInfo.TargetDomainBones = LocalDebugInfo.TargetDomainBones;

	DebugDrawInfo.TargetGoals = LocalDebugInfo.TargetGoals;
	DebugDrawInfo.PairRetargetInfo = LocalDebugInfo.PairRetargetInfo;
	DebugDrawInfo.SourcePairVerts = LocalDebugInfo.SourcePairVerts;
	DebugDrawInfo.TargetPairVerts = LocalDebugInfo.TargetPairVerts;

	// Cache debug info for source bodies
	DebugDrawInfo.SourceBodyInfo.Reset(CacheSourceEffectBones.Num());
	DebugDrawInfo.TargetBodyInfo.Reset(CacheSourceEffectBones.Num());
	for (int SourceEffectBoneIndex = 0; SourceEffectBoneIndex<CacheSourceEffectBones.Num(); SourceEffectBoneIndex++)
	{
		FName SourceBoneName = CacheSourceEffectBones[SourceEffectBoneIndex];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		
		if (CacheSourceSkelIndices.Contains(SourceBoneName))
		{
			int32 SourcePoseIdx = CacheSourceSkelIndices[SourceBoneName];
			// Need to apply source-scale to bodies since it's directly applied to translations only in source pose
			FTransform SourceScaleTfm{FQuat::Identity, FVector::ZeroVector, FVector(SourceScale)};
			DebugDrawInfo.SourceBodyInfo.Emplace(SourceBoneName, SourceScale, SourceScaleTfm*SourcePose[SourcePoseIdx], OffsetTransformsForSourceBones(SourceBoneName));
		}
		
		if (CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			int32 TargetPoseIdx = CacheTargetSkelIndices[TargetBoneName];
			DebugDrawInfo.TargetBodyInfo.Emplace(TargetBoneName, 1.0, TargetPose[TargetPoseIdx], FTransform::Identity);
		}
	}

	DebugDrawInfo.SourceTfmInfo.Reset(CacheSourceEffectBones.Num());
	DebugDrawInfo.TargetTfmInfo.Reset(CacheSourceEffectBones.Num());
	for (int SourceEffectBoneIndex = 0; SourceEffectBoneIndex<CacheSourceEffectBones.Num(); SourceEffectBoneIndex++)
	{
		FName SourceBoneName = CacheSourceEffectBones[SourceEffectBoneIndex];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		
		if (CacheSourceSkelIndices.Contains(SourceBoneName))
		{
			int32 SourcePoseIdx = CacheSourceSkelIndices[SourceBoneName];
		
			FBodyTransform SourceFullTfm;
			FTransform SourceBodyRot(OffsetTransformsForSourceBones(SourceBoneName).GetRotation());
			ComputeSourceBodyTransform(SourceFullTfm, SourceBoneName, SourcePose[SourcePoseIdx], SourceScale);

			FVector SourceBodyCenter = ApplyBodyTransform(SourceFullTfm, FVector::ZeroVector);
			FVector SourceBodyX = ApplyBodyTransform(SourceFullTfm, SourceBodyRot.GetUnitAxis(EAxis::X));
			FVector SourceBodyY = ApplyBodyTransform(SourceFullTfm, SourceBodyRot.GetUnitAxis(EAxis::Y));
			FVector SourceBodyZ = ApplyBodyTransform(SourceFullTfm, SourceBodyRot.GetUnitAxis(EAxis::Z));
		
			DebugDrawInfo.SourceTfmInfo.Emplace(SourceBodyCenter, SourceBodyX, SourceBodyY, SourceBodyZ);
		}

		if (CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			int32 TargetPoseIdx = CacheTargetSkelIndices[TargetBoneName];

			FBodyTransform TargetFullTfm;
			FTransform SourceBodyRot(OffsetTransformsForSourceBones(SourceBoneName).GetRotation());
			ComputeTargetBodyTransform(TargetFullTfm, SourceBoneName, TargetPose[TargetPoseIdx]);

			FVector TargetBodyCenter = ApplyBodyTransform(TargetFullTfm, FVector::ZeroVector);
			FVector TargetBodyX = ApplyBodyTransform(TargetFullTfm, SourceBodyRot.GetUnitAxis(EAxis::X));
			FVector TargetBodyY = ApplyBodyTransform(TargetFullTfm, SourceBodyRot.GetUnitAxis(EAxis::Y));
			FVector TargetBodyZ = ApplyBodyTransform(TargetFullTfm, SourceBodyRot.GetUnitAxis(EAxis::Z));
		
			DebugDrawInfo.TargetTfmInfo.Emplace(TargetBodyCenter, TargetBodyX, TargetBodyY, TargetBodyZ);
		}
	}
}

void FIKRetargetRelativeIKOp::ResetDebugInfo()
{
	FScopeLock ScopeLock(&DebugDataMutex);
	
	DebugDrawInfo.SourcePairVerts.Reset();
	DebugDrawInfo.TargetPairVerts.Reset();
	
	DebugDrawInfo.SourceBodyInfo.Reset();
	DebugDrawInfo.TargetBodyInfo.Reset();

	DebugDrawInfo.SourceTfmInfo.Reset();
	DebugDrawInfo.TargetTfmInfo.Reset();
	
	DebugDrawInfo.PairRetargetInfo.Reset();
	DebugDrawInfo.TargetGoals.Reset();
}

#endif //WITH_EDITOR

FIKRetargetRelativeIKOpSettings UIKRetargetRelativeIKController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetRelativeIKOpSettings*>(OpSettingsToControl);
}

void UIKRetargetRelativeIKController::SetSettings(FIKRetargetRelativeIKOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
