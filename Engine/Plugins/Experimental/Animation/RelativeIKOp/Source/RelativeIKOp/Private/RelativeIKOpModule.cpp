// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelativeIKOpModule.h"

#define LOCTEXT_NAMESPACE "RelativeIKOpModule"

void FRelativeIKOpModule::StartupModule()
{
}

void FRelativeIKOpModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FRelativeIKOpModule, RelativeIKOp)