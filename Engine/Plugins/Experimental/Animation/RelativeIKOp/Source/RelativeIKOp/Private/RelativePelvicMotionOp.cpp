// Copyright Epic Games, Inc. All Rights Reserved.
#include "RelativePelvicMotionOp.h"

// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelativeIKOp.h"

#include "IKRigDebugRendering.h"
#include "RelativeBodyAnimNotifies.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Components/SkeletalMeshComponent.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"
#include "Eigen/SVD"

#if WITH_EDITOR
#include "PrimitiveDrawingUtils.h"
#include "Engine/EngineTypes.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RelativePelvicMotionOp)

#define LOCTEXT_NAMESPACE "RelativePelvicMotionOp"

// TODO: Merge this into solver when we put this stuff together
template<int P> double RationalSmoothStepdown(double Distance, double DistThreshold, double DistHalfPointLambda);

const UClass* FIKRetargetPelvicMotionOpSettings::GetControllerType() const
{
	return UIKRetargetRelativeIKController::StaticClass();
}

void FIKRetargetPelvicMotionOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	static TArray<FName> PropertiesToIgnore = {};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetPelvicMotionOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

bool FIKRetargetPelvicMotionOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;
	
	TargetPhysicsAsset = Settings.TargetPhysicsAssetOverride;
	if (!TargetPhysicsAsset)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingPhysicsAssets", "{0}: Target Physics Asset must be specified. "), FText::FromName(GetName())));
		return false;
	}
	
	int32 TargetPelvisIndex = InTargetSkeleton.FindBoneIndexByName(Settings.TargetPelvicBone);
	if (Settings.TargetPelvicBone == NAME_None || TargetPelvisIndex == INDEX_NONE)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("InvalidPelvicBone", "{0}: Valid pelvic bone must be specified. "), FText::FromName(GetName())));
		return false;
	}
	
	SourcePelvicBone = NAME_None;
	if (InSourceSkeleton.FindBoneIndexByName(Settings.TargetPelvicBone) != INDEX_NONE)
	{
		SourcePelvicBone = Settings.TargetPelvicBone;
	}
	else
	{
		for (const TTuple<FName,FName>& BoneMap : Settings.BodyMapping)
		{
			if (BoneMap.Value == Settings.TargetPelvicBone)
			{
				SourcePelvicBone = BoneMap.Key;
			}
		}
	}
	
	// Verify source/target have same name or there's a valid mapping to target
	// TODO: We verify that source with same name isn't mapped to none by relik convention but could remove this later
	if (SourcePelvicBone == NAME_None || ApplyBodyMap(SourcePelvicBone) != Settings.TargetPelvicBone)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("InvalidPelvicMapping", "{0}: Valid source/target mapping for pelvic bone must be specified. "), FText::FromName(GetName())));
		return false;
	}
	
	UpdateCacheSkelInfo(InSourceSkeleton, InTargetSkeleton);
	
	// Add all pin bones to the source/target skel lookup tables
	InitializePinData(InProcessor);

	// Force update cache info if new anim montage found
	MontageInstance = nullptr;
	CacheSourceAnimSequence = nullptr;
	
	bIsInitialized = true;
	return bIsInitialized;
}

void FIKRetargetPelvicMotionOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
#if WITH_EDITOR
	ResetDebugInfo();
#endif //WITH_EDITOR
	
	if (InProcessor.IsIKForcedOff())
	{
		return;
	}
	
	if (!bIsInitialized || !CacheSourceAnimSequence || AnimSeqPlayHead < 0.0f)
	{
		return;
	}
	
	// TODO: Figure out how to move these out of processing functions?
	FDebugRelativeIKDrawInfo LocalDebugInfo;
	TArray<FDebugRelativeTargetPairSpace> LocalTargetPairSpaces;

	// int32 NumMeshPairs = (CachedNotifyInfo) ? CachedNotifyInfo->bBodyPairsIsParentDominates.Num() : 0;
	int32 NumPropPairs = (CachedPropNotifyInfo) ? CachedPropNotifyInfo->bPropsPairsIsParentDominates.Num() : 0;
	// int32 TotalPairs = NumMeshPairs + NumPropPairs;
	int32 TotalPairs = NumPropPairs;
	
	// int32 NumMeshBodies = (CachedNotifyInfo) ? CachedNotifyInfo->BodyPairs.Num() : 0;
	int32 NumPropBones = (CachedPropNotifyInfo) ? CachedPropNotifyInfo->PropsPairs.Num() : 0;
	// int32 TotalBodies = NumMeshBodies + NumPropBones;
	int32 TotalBodies = NumPropBones;

	if ( TotalBodies == 0 )
	{
		return;
	}
	
#if WITH_EDITOR
	LocalDebugInfo.SourcePairVerts.Reserve(TotalPairs);
	LocalDebugInfo.TargetPairVerts.Reserve(TotalPairs);
	
	LocalTargetPairSpaces.SetNumZeroed(TotalBodies);
#endif //WITH_EDITOR

	// Copy/cache (pin) prop bones in target using pin bone settings
	CachePinBoneTransforms(InProcessor, InSourceGlobalPose, OutTargetGlobalPose);

	const double SourceScale = InProcessor.GetSourceScaleFactor();
	// TODO: Do we want all or some body-pairs to contribute to pelvis motion?
	// if (CachedNotifyInfo)
	// {
	// 	int32 NumMeshSamples = CachedNotifyInfo->NumSamples;
	// 	TConstArrayView<FVector3f> FramePointView = ApplyTemporalSmoothing(AnimSeqPlayHead, MeshSampleRate, NumMeshSamples, NumMeshBodies, CachedNotifyInfo->BodyPairsLocalReference);
	// 	ComputeFramePairSpaces(SourceScale, InSourceGlobalPose, OutTargetGlobalPose, CacheMeshPairOffset, FramePointView, LocalDebugInfo, LocalTargetPairSpaces);
	// }

	if (CachedPropNotifyInfo)
	{
		int32 NumPropSamples = CachedPropNotifyInfo->NumSamples;
		TConstArrayView<FVector3f> FramePropView = ApplyTemporalSmoothing(AnimSeqPlayHead, PropSampleRate, NumPropSamples, NumPropBones, CachedPropNotifyInfo->PropsPairsLocalReference);
		ComputeFramePropSpaces(SourceScale, InSourceGlobalPose, OutTargetGlobalPose, CachePropPairOffset, FramePropView, LocalDebugInfo, LocalTargetPairSpaces);
	}
	
	LocalDebugInfo.TargetGoals.Reserve(CacheSourceBodyEffectVertIdx.Num());
	
	// Update pelvis bone
	ApplyAverageBodyPairTargets(OutTargetGlobalPose, InProcessor, LocalDebugInfo, LocalTargetPairSpaces);
	
	// Undo check attach bone changes
	// TODO: Clean this up and normalize between relik and relpelvic ops
	for (const FRelativeIKPinBoneData& PinData : CachePinBoneData)
	{
		FName SourcePinBone = PinData.SourceBone;
		FName TargetPinBone = PinData.TargetBone;
		if (!CacheSourceSkelIndices.Contains(SourcePinBone) || !CacheTargetSkelIndices.Contains(TargetPinBone))
		{
			continue;
		}
		
		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetPinBone];
		int32 CachePropIdx = CachePinSourcePropBones.Find(SourcePinBone);
		if (CachePrevPinTargetPropTransforms.IsValidIndex(CachePropIdx))
		{
			OutTargetGlobalPose[TargetBoneIdx] = CachePrevPinTargetPropTransforms[CachePropIdx];
		}
	}

#if WITH_EDITOR
	LocalDebugInfo.TargetDomainBones.Reset(CacheSourceEffectBones.Num());
	// for (int PairIdx=0; PairIdx<NumMeshPairs; PairIdx++)
	// {
	// 	if(CachedNotifyInfo->bBodyPairsIsParentDominates[PairIdx])
	// 	{
	// 		FName DomainBody = ApplyBodyMap(CachedNotifyInfo->BodyPairs[2*PairIdx]);
	// 		if (!LocalDebugInfo.TargetDomainBones.Contains(DomainBody))
	// 		{
	// 			LocalDebugInfo.TargetDomainBones.Add(DomainBody);
	// 		}
	// 	}
	// }
	for (int PairIdx=0; PairIdx<NumPropPairs; PairIdx++)
	{
		if(CachedPropNotifyInfo->bPropsPairsIsParentDominates[PairIdx])
		{
			FName DomainBody = ApplyBodyMap(CachedPropNotifyInfo->PropsPairs[2*PairIdx+1]);
			if (!LocalDebugInfo.TargetDomainBones.Contains(DomainBody))
			{
				LocalDebugInfo.TargetDomainBones.Add(DomainBody);
			}
		}
	}
	UpdateDebugInfo(SourceScale, InSourceGlobalPose, OutTargetGlobalPose, LocalDebugInfo);
#endif
}

void FIKRetargetPelvicMotionOp::ComputeFramePropSpaces(
	double SourceScale,
	const TArray<FTransform>& SourcePose,
	const TArray<FTransform>& TargetPose,
	int32 PairOffset,
	TConstArrayView<FVector3f> FramePointView,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	int32 PairCount = CachedPropNotifyInfo->bPropsPairsIsParentDominates.Num();
	if (PairCount <= 0)
	{
		return;
	}

	FTransform SourceScaleTfm(FQuat::Identity, FVector::ZeroVector, FVector(SourceScale));
	const double InvSourceScale = (Settings.bIgnoreSourceScale) ? 1.0 / SourceScale : 1.0;
	
	const double RetargetContactAlpha = FMath::Clamp(Settings.RetargetContactAlpha, 0.0, 1.0);
	const double RetargetSpringAlpha = FMath::Clamp(Settings.RetargetSpringAlpha, 0.0, 1.0);
	// Get pair target body verts and weights
	int32 PelvicEffectIdx = CacheSourceEffectBones.Find(SourcePelvicBone);
	if (PelvicEffectIdx == INDEX_NONE)
	{
		return;
	}
	
	// Only run through pelvic effect indices
	for (int32 OutEffectIdx : CacheSourceBodyEffectVertIdx[PelvicEffectIdx])
	{
		int32 OutPairIdx = OutEffectIdx / 2;
		int32 PairIdx = OutPairIdx - PairOffset;
		if (!CachedPropNotifyInfo->bPropsPairsIsParentDominates.IsValidIndex(PairIdx))
		{
			continue;
		}
		
		FrameBoneConstraints[2*OutPairIdx].Reset();
		FrameBoneConstraints[2*OutPairIdx+1].Reset();
		
		FName SourcePropBone = CachedPropNotifyInfo->PropsPairs[2 * PairIdx];
		FName TargetPropBone = ApplyBodyMap(SourcePropBone);
		FName SourceBodyBone = CachedPropNotifyInfo->PropsPairs[2*PairIdx + 1];
		FName TargetBodyBone = ApplyBodyMap(SourceBodyBone);

		if (TargetPropBone == NAME_None || !CacheSourceSkelIndices.Contains(SourcePropBone) || !CacheTargetSkelIndices.Contains(TargetPropBone) )
		{
			continue;
		}
		
		if (TargetBodyBone == NAME_None || !CacheSourceSkelIndices.Contains(SourceBodyBone) || !CacheTargetSkelIndices.Contains(TargetBodyBone))
		{
			continue;
		}

		if (!HasBoneDelta(TargetPropBone) || !HasBoneDelta(TargetBodyBone))
		{
			continue;
		}
		
		int32 SourcePropPoseIdx = CacheSourceSkelIndices[SourcePropBone];
		int32 SourceBodyPoseIdx = CacheSourceSkelIndices[SourceBodyBone];
		
		// Compute source transforms and source-space vertex distance
		// const FVector3f& LocalPropVert = FramePointView[2*PairIdx];
		// const FVector3f& LocalBodyVert = FramePointView[2*PairIdx+1];
		FVector SourceBoneBakePropV(FramePointView[2*PairIdx]);
		FVector SourceBoneBakeBodyV(FramePointView[2*PairIdx+1]);
		
		FTransform SourcePropTfm = SourceScaleTfm * SourcePose[SourcePropPoseIdx];
		FBodyTransform SourceBodyTfm;
		ComputeSourceBodyTransform(SourceBodyTfm, SourceBodyBone, SourcePose[SourceBodyPoseIdx], SourceScale);
		
		FVector SourcePropCompV = SourcePropTfm.TransformPosition(SourceBoneBakePropV);
		FVector SourceBodyCompV = ApplyBodyTransform(SourceBodyTfm, SourceBoneBakeBodyV);

		// Rescale components for distance checks if "Ignore sources scale" is on
		// NOTE: This works because we don't need absolute coordinates (just distance).
		//       If absolute coords needed then must properly account for scale pivot
		FVector RescaleSourcePropV = InvSourceScale * SourcePropCompV;
		FVector RescaleSourceBodyV = InvSourceScale * SourceBodyCompV;

		double SourceVDist = FVector::Distance(RescaleSourcePropV, RescaleSourceBodyV);
		double LinearWeight = FMath::Clamp(1.0 - (SourceVDist / Settings.DistanceThreshold), 0.0, 1.0);
		double Weight = GetDistanceWeight(Settings.DistanceWeightMode, SourceVDist, Settings.DistanceThreshold, Settings.DistHalfPointLambda);

		// Compute Source pair verts as represented in alternate pair body primary(_P) or other(_O) local bake space
		FVector SourcePropV_B = InverseBodyTransform(SourceBodyTfm, SourcePropCompV);
		FVector SourceBodyV_P = SourcePropTfm.InverseTransformPosition(SourceBodyCompV);
		
		//---------------------------------
		// Get verts retargeted using physics body oriented scale
		int32 TargetPropPoseIdx = CacheTargetSkelIndices[TargetPropBone];
		int32 TargetBodyPoseIdx = CacheTargetSkelIndices[TargetBodyBone];

		FTransform TargetPropBoneDelta = GetTargetBoneDelta(TargetPropBone);

		FTransform TargetPropTfm = TargetPropBoneDelta * TargetPose[TargetPropPoseIdx];
		FBodyTransform TargetBodyTfm;
		ComputeTargetBodyTransform(TargetBodyTfm, TargetBodyBone, TargetPose[TargetBodyPoseIdx]);

		// Compute prop vert represented in component space relative to prop (_P) and body (_B)
		FVector TargetPropCompV_P = TargetPropTfm.TransformPosition(SourceBoneBakePropV);
		FVector TargetPropCompV_B = ApplyBodyTransform(TargetBodyTfm, SourcePropV_B);

		// Compute body vert represented in component space relative to prop (_P) and body (_B)
		FVector TargetBodyCompV_B = ApplyBodyTransform(TargetBodyTfm, SourceBoneBakeBodyV);
		FVector TargetBodyCompV_P = TargetPropTfm.TransformPosition(SourceBodyV_P);
		
		// These are the target midpoint positions used for spring constraints in PhysicalIK
		FVector TargetSpring_P = FMath::Lerp(TargetPropCompV_P,TargetBodyCompV_P, RetargetSpringAlpha);
		FVector TargetSpring_B = FMath::Lerp(TargetPropCompV_B, TargetBodyCompV_B, RetargetSpringAlpha);

		// Offset vector (Body -> Prop) to Lerp for dist alpha
		FVector TargetOffsetVec = TargetSpring_P - TargetSpring_B;

		FVector TargetPropBoneLoc = TargetPose[TargetPropPoseIdx].GetLocation();
		FVector TargetBodyBoneLoc = TargetPose[TargetBodyPoseIdx].GetLocation();
		
		FDoubleInterval FeasibleRange_P(0.0, 1.0);
		FDoubleInterval FeasibleRange_B(0.0, 1.0);

		// double ContactBlendWeight = LinearWeight;
		double ContactBlendWeight = Settings.RetargetContactAlpha;
		double FeasibilityWeight_B =  FMath::Lerp(FeasibleRange_B.Min, FeasibleRange_B.Max, ContactBlendWeight);
		double FeasibilityWeight_P = 0.0;
		
		FVector WeightedTargetOffset_B = TargetOffsetVec * FeasibilityWeight_B;
		FVector WeightedTargetOffset_P = -TargetOffsetVec * FeasibilityWeight_P;

		FrameBoneConstraints[2*OutPairIdx].Offset = WeightedTargetOffset_P;
		FrameBoneConstraints[2*OutPairIdx+1].Offset = WeightedTargetOffset_B;
		FrameBoneConstraints[2*OutPairIdx].Weight = Weight;
		FrameBoneConstraints[2*OutPairIdx+1].Weight = Weight;

#if WITH_EDITOR
		// Debug draw verts
		if (Settings.bDebugDraw)
		{
			DebugInfo.SourcePairVerts.Add({SourcePropCompV, SourceBodyCompV, false, FVector::ZeroVector, FVector::ZeroVector, Weight});
			if (Settings.DebugFullRetargetPairBones.Contains(TargetPropBone))
			{
				DebugInfo.TargetPairVerts.Add({TargetPropCompV_P, TargetBodyCompV_B, true,TargetBodyCompV_P, TargetPropCompV_B, Weight});
			}
			else
			{
				DebugInfo.TargetPairVerts.Add({TargetPropCompV_P, TargetBodyCompV_B, false, FVector::ZeroVector, FVector::ZeroVector, Weight});
			}
			
			DebugPairSpaces[2*OutPairIdx] = FDebugRelativeTargetPairSpace{TargetPropCompV_P, TargetPropCompV_P-TargetOffsetVec, FeasibilityWeight_P, FeasibleRange_P};
			DebugPairSpaces[2*OutPairIdx+1] = FDebugRelativeTargetPairSpace{TargetBodyCompV_B, TargetBodyCompV_B+TargetOffsetVec, FeasibilityWeight_B, FeasibleRange_B};
		}
#endif //WITH_EDITOR
	}
}

void FIKRetargetPelvicMotionOp::InitializePinData(const FIKRetargetProcessor& Processor)
{
	// get skeletons we are copying from/to
	const FRetargetSkeleton& SourceSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Source);
	const FRetargetSkeleton& TargetSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Target);

	// get retarget pose of source and target
	const TArray<FTransform>& SourceRetargetRefPose = SourceSkeleton.RetargetPoses.GetGlobalRetargetPose();
	const TArray<FTransform>& TargetRetargeRefPose = TargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
	
	CachePinBoneData.Reset(Settings.CheckPropBones.Num());
	for (const FPinBoneSettings& PinBoneInfo : Settings.CheckPropBones)
	{
		FName SourceBoneName = PinBoneInfo.SourceBoneName;
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		if (TargetBoneName == NAME_None)
		{
			continue;
		}
		
		UpdateSourceBoneMap(SourceBoneName);
		UpdateTargetBoneMap(TargetBoneName);
		
		if (!CacheSourceSkelIndices.Contains(SourceBoneName) || !CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			continue;
		}
		
		// get relevant transforms from the retarget poses
		int32 PropBoneIdx = CacheSourceSkelIndices[SourceBoneName];
		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
		
		FTransform PropBoneParentRefPoseGlobal = GetParentTransform(SourceSkeleton, PropBoneIdx, SourceRetargetRefPose);
		FTransform PropBoneRefPoseGlobal = SourceRetargetRefPose[PropBoneIdx];
		
		FTransform TargetBoneParentRefPoseGlobal = GetParentTransform(TargetSkeleton, TargetBoneIdx, TargetRetargeRefPose);
		FTransform TargetBoneRefPoseGlobal = TargetRetargeRefPose[TargetBoneIdx];
		
		FRelativeIKPinBoneData PinData;
		PinData.TranslationMode = PinBoneInfo.TranslationMode;
		PinData.RotationMode = PinBoneInfo.RotationMode;
		
		PinData.SourceBone = SourceBoneName;
		PinData.TargetBone = TargetBoneName;
		
		PinData.SourceTargetOffsetTfm = PropBoneRefPoseGlobal.GetRelativeTransform(TargetBoneRefPoseGlobal);
		PinData.SourceLocalRefTfm = PropBoneParentRefPoseGlobal.GetRelativeTransform(PropBoneRefPoseGlobal);
		PinData.TargetLocalRefTfm = TargetBoneParentRefPoseGlobal.GetRelativeTransform(TargetBoneRefPoseGlobal);
		PinData.RefRotDelta = TargetBoneRefPoseGlobal.GetRotation() * PropBoneRefPoseGlobal.GetRotation().Inverse();
		
		CachePinBoneData.Add(PinData);
	}
}

void FIKRetargetPelvicMotionOp::CachePinBoneTransforms(
	const FIKRetargetProcessor& Processor,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	CachePrevPinTargetPropTransforms.SetNumZeroed(CachePinSourcePropBones.Num());
	for (const FRelativeIKPinBoneData& PinData : CachePinBoneData)
	{
		FName SourcePropBone = PinData.SourceBone;
		// Will calculate non-cached pins in ApplyAfterParent
		int32 CachePropIdx = CachePinSourcePropBones.Find(SourcePropBone);
		if (CachePropIdx == INDEX_NONE)
		{
			continue;
		}
		
		FName TargetPropBone = ApplyBodyMap(SourcePropBone);
		if (!CacheSourceSkelIndices.Contains(SourcePropBone) || !CacheTargetSkelIndices.Contains(TargetPropBone))
		{
			continue;
		}
		
		int32 TargetBoneIdx = CacheTargetSkelIndices[TargetPropBone];
		CachePrevPinTargetPropTransforms[CachePropIdx] = OutTargetGlobalPose[TargetBoneIdx];
		FTransform OutGlobalTransform = ComputePinBoneTransform(PinData, Processor, InSourceGlobalPose, OutTargetGlobalPose);
		
		OutTargetGlobalPose[TargetBoneIdx] = OutGlobalTransform;
	}
}

FTransform FIKRetargetPelvicMotionOp::ComputePinBoneTransform(
		const FRelativeIKPinBoneData& PinData,
		const FIKRetargetProcessor& Processor,
		const TArray<FTransform>& InSourceGlobalPose,
		const TArray<FTransform>& OutTargetGlobalPose)
{
	int32 SourceBoneIdx = CacheSourceSkelIndices[PinData.SourceBone];
	int32 TargetBoneIdx = CacheTargetSkelIndices[PinData.TargetBone];
	
	const FRetargetSkeleton& SourceSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Source);
	const FRetargetSkeleton& TargetSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Target);
	
	const FTransform& SourceTransform = InSourceGlobalPose[SourceBoneIdx]; 
	FTransform TargetTransform = SourceTransform;
	
	// get the current translational offset of BoneToCopyFrom relative to it's parent
	auto GetSourceLocalBoneVector = [SourceSkeleton, SourceBoneIdx, InSourceGlobalPose]()
	{
		const FTransform& SourceTransform = InSourceGlobalPose[SourceBoneIdx];
		const FTransform& ParentTransform = GetParentTransform(SourceSkeleton, SourceBoneIdx, InSourceGlobalPose);
		return SourceTransform.GetTranslation() - ParentTransform.GetTranslation();
	};
	
	// generate translational offset
	switch (PinData.TranslationMode)
	{
	case EPinBoneTranslationMode::CopyGlobalPosition:
		{
			TargetTransform.SetTranslation(SourceTransform.GetTranslation());
			break;
		}
		
	case EPinBoneTranslationMode::CopyGlobalPositionAndMaintainOffset:
		{
			TargetTransform.SetTranslation((PinData.SourceTargetOffsetTfm * SourceTransform).GetTranslation());
			break;
		}
		
	case EPinBoneTranslationMode::CopyLocalPosition:
		{
			const FVector SourceLocalVector = GetSourceLocalBoneVector();
			const FTransform& TargetParentTfm = GetParentTransform(TargetSkeleton, TargetBoneIdx, OutTargetGlobalPose);
			TargetTransform.SetTranslation(TargetParentTfm.GetTranslation() + SourceLocalVector);
			break;
		}
		
	case EPinBoneTranslationMode::CopyLocalPositionRelativeOffset:
		{
			const FVector SourceLocalVector = GetSourceLocalBoneVector();
			double SourceBoneLength;
			FVector SourceTranslationDir;
			SourceLocalVector.ToDirectionAndLength(SourceTranslationDir, SourceBoneLength);
			
			const double RestPoseLengthDifference = PinData.TargetLocalRefTfm.GetTranslation().Size() - PinData.SourceLocalRefTfm.GetTranslation().Size();
			const FVector SourceRelativeOffset = SourceTranslationDir * (SourceBoneLength + RestPoseLengthDifference);
		
			const FTransform& ParentOfBoneToCopyToCurrent = GetParentTransform(TargetSkeleton, TargetBoneIdx, OutTargetGlobalPose);
			TargetTransform.SetTranslation(ParentOfBoneToCopyToCurrent.GetTranslation() + SourceRelativeOffset);
			break;
		}
		
	case EPinBoneTranslationMode::CopyLocalPositionRelativeScaled:
		{
			const FVector SourceLocalVector = GetSourceLocalBoneVector();
			double SourceBoneLength;
			FVector SourceTranslationDir;
			SourceLocalVector.ToDirectionAndLength(SourceTranslationDir, SourceBoneLength);
			
			const double SourceRefBoneLength = PinData.SourceLocalRefTfm.GetTranslation().Size();
			const double TargetRefBoneLength = PinData.TargetLocalRefTfm.GetTranslation().Size();
			const double BoneLengthScaleFactor = (!FMath::IsNearlyZero(SourceRefBoneLength)) ? TargetRefBoneLength / SourceRefBoneLength : 0.0;
			
			const FVector BoneToCopyFromRelativeOffset = SourceTranslationDir * (SourceBoneLength * BoneLengthScaleFactor);
		
			const FTransform& ParentOfBoneToCopyToCurrent = GetParentTransform(TargetSkeleton, TargetBoneIdx, OutTargetGlobalPose);
			TargetTransform.SetTranslation(ParentOfBoneToCopyToCurrent.GetTranslation() + BoneToCopyFromRelativeOffset);
			break;
		}
		
	default:
		ensureAlwaysMsgf(false, TEXT("Missing translation offset mode."));
		break;
	}

	// generate rotational offset
	switch (PinData.RotationMode)
	{
	case EPinBoneRotationMode::CopyGlobalRotation:
		{
			TargetTransform.SetRotation(SourceTransform.GetRotation());
			break;
		}
	case EPinBoneRotationMode::MaintainOffsetFromBoneToCopyFrom:
		{
			// apply "copy from" rotation plus delta
			TargetTransform.SetRotation(PinData.RefRotDelta * SourceTransform.GetRotation());
			break;
		}
		
	default:
		ensureAlwaysMsgf(false, TEXT("Missing rotation offset mode."));
		break;
	}
	
	return TargetTransform;
}

const FTransform& FIKRetargetPelvicMotionOp::GetParentTransform(const FRetargetSkeleton& Skeleton, const int32 BoneIndex, const TArray<FTransform>& InPose)
{
	int32 ParentIdx = Skeleton.GetParentIndex(BoneIndex);
	if (ParentIdx == INDEX_NONE)
	{
		return FTransform::Identity;
	}
	
	return InPose[ParentIdx];
}

void FIKRetargetPelvicMotionOp::ApplyAverageBodyPairTargets(
	TArray<FTransform>& OutTargetPose,
	FIKRetargetProcessor& InProcessor,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	int32 FindEffectIdx = CacheSourceEffectBones.Find(SourcePelvicBone);
	if (!CacheSourceBodyEffectVertIdx.IsValidIndex(FindEffectIdx))
	{
		return;
	}
		
	TArray<int32>& EffectIndices = CacheSourceBodyEffectVertIdx[FindEffectIdx];
	if (EffectIndices.IsEmpty())
	{
		return;
	}

	FName SourceBoneName = CacheSourceEffectBones[FindEffectIdx];
	FName TargetBoneName = ApplyBodyMap(SourceBoneName);
	if (!CacheTargetSkelIndices.Contains(TargetBoneName))
	{
		return;
	}

	int32 TargetBoneIdx = CacheTargetSkelIndices[TargetBoneName];
	FTransform& TargetPelvicBonePose = OutTargetPose[TargetBoneIdx];
	FVector BoneLoc = TargetPelvicBonePose.GetLocation();

	FVector TargetOffset;
	FVector TotalWeight = ComputeTargetWeightedOffset(TargetOffset, EffectIndices, BoneLoc, DebugInfo, DebugPairSpaces);
	if (FMath::IsNearlyZero(TotalWeight.GetMax()))
	{
		return;
	}
	
	UpdateTargetPelvisBone(OutTargetPose, TargetBoneIdx, InProcessor, TotalWeight, TargetOffset);
}

void FIKRetargetPelvicMotionOp::UpdateTargetPelvisBone(
	TArray<FTransform>& OutTargetPose,
	int32 TargetBoneIdx,
	FIKRetargetProcessor& InProcessor,
	const FVector& TargetWeight,
	const FVector& TargetGoalOffset)
{
	FTransform PelvisTfm = OutTargetPose[TargetBoneIdx];
	
	// TODO: Play with total weight functions to alpha goal
	FVector GoalAlpha = FVector::Min(FVector::OneVector, TargetWeight*FMath::Clamp(Settings.ContributionSumWeight, 0.0f, 1.0f));
	FVector TargetGoalLoc = PelvisTfm.GetLocation() + TargetGoalOffset * GoalAlpha;
	
	// TODO: Remove dry run for release
	bool bSetGoals = !Settings.bDebugDraw || !Settings.bDryRun;
	if (bSetGoals)
	{
		PelvisTfm.SetLocation(TargetGoalLoc);
		
		FTargetSkeleton& TargetSkeleton = InProcessor.GetTargetSkeleton();
		TargetSkeleton.SetGlobalTransformAndUpdateChildren(TargetBoneIdx, PelvisTfm, OutTargetPose);
	}
}

FVector FIKRetargetPelvicMotionOp::ComputeTargetWeightedOffset(
	FVector& OutTargetOffset,
	const TArray<int32>& EffectIndices,
	const FVector& BoneLoc,
	FDebugRelativeIKDrawInfo& DebugInfo,
	TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces)
{
	FVector TotalWeight = FVector::ZeroVector;
	OutTargetOffset = FVector::ZeroVector;

#if WITH_EDITOR
	if (Settings.bDebugDraw)
	{
		DebugInfo.TargetGoals.AddDefaulted();
		DebugInfo.TargetGoals.Last().Goal = FVector::ZeroVector;
	}
#endif //WITH_EDITOR
	
	for (int Index : EffectIndices)
	{
		const FRelativeIKFrameBoneConstraint& BoneConstraint = FrameBoneConstraints[Index];
		if (FMath::IsNearlyZero(BoneConstraint.Weight))
		{
			continue;
		}

		FVector TargetOffset = BoneConstraint.Offset;
		FVector TargetWeight = FVector(BoneConstraint.Weight);
		if (BoneConstraint.ConstraintType == ERelativeIKFrameBoneConstraintType::Z)
		{
			TargetWeight = {0,0,BoneConstraint.Weight};
		}

		OutTargetOffset += TargetWeight * TargetOffset;
		TotalWeight += TargetWeight;

#if WITH_EDITOR
		if (Settings.bDebugDraw)
		{
			// TODO: Fixup target debug to show dynamic offset goals (post-solve update?)
			FVector BoneTarget = TargetOffset + BoneLoc;
			DebugInfo.TargetGoals.Last().PairTargets.Add(BoneTarget);
			DebugInfo.PairRetargetInfo.Emplace(DebugPairSpaces[Index]);
		}
#endif //WITH_EDITOR
	}
	
	FVector InvWeight = {
		(FMath::IsNearlyZero(TotalWeight.X) ? 0.0 : 1.0 / TotalWeight.X),
		(FMath::IsNearlyZero(TotalWeight.Y) ? 0.0 : 1.0 / TotalWeight.Y),
		(FMath::IsNearlyZero(TotalWeight.Z) ? 0.0 : 1.0 / TotalWeight.Z),

	};
	OutTargetOffset *= InvWeight;
	
#if WITH_EDITOR
	if (Settings.bDebugDraw)
	{
		DebugInfo.TargetGoals.Last().Goal = OutTargetOffset + BoneLoc;
	}
#endif // WITH_EDITOR
	
	return TotalWeight;
}

FName FIKRetargetPelvicMotionOp::ApplyBodyMap(FName BodyName) const
{
	if (Settings.BodyMapping.IsEmpty() || !Settings.BodyMapping.Contains(BodyName))
	{
		return BodyName;
	}

	return Settings.BodyMapping[BodyName];
}

bool FIKRetargetPelvicMotionOp::HasBoneDelta(FName TargetBoneName) const
{
	return CacheMapBoneSourceTargetTfm.Contains(TargetBoneName);
}

const FTransform& FIKRetargetPelvicMotionOp::GetTargetBoneDelta(FName TargetBoneName) const
{
	if (!CacheMapBoneSourceTargetTfm.Contains(TargetBoneName))
	{
		return FTransform::Identity;
	}
	return CacheMapBoneSourceTargetTfm[TargetBoneName];
}

const FTransform& FIKRetargetPelvicMotionOp::OffsetTransformsForSourceBones(FName SourceBoneName) const
{
	// if (CachedNotifyInfo && CachedNotifyInfo->OffsetTransformsForBones.Contains(SourceBoneName))
	// {
	// 	return CachedNotifyInfo->OffsetTransformsForBones[SourceBoneName];
	// }

	if (CachedPropNotifyInfo && CachedPropNotifyInfo->OffsetTransformsForBones.Contains(SourceBoneName))
	{
		return CachedPropNotifyInfo->OffsetTransformsForBones[SourceBoneName];
	}
	
	return FTransform::Identity;
}

void FIKRetargetPelvicMotionOp::ComputeSourceBodyTransform(FBodyTransform& OutTransform, FName SourceBoneName, const FTransform& GlobalTfm, double SourceScale) const
{
	FTransform BoneOffsetTfm = OffsetTransformsForSourceBones(SourceBoneName);
	FQuat BodyRot = BoneOffsetTfm.GetRotation();
	FVector BodyTrans = BoneOffsetTfm.GetTranslation() * SourceScale;
	FVector BodyScale = BoneOffsetTfm.GetScale3D() * SourceScale;

	OutTransform.BoneToBody = FTransform(BodyRot.Inverse());
	OutTransform.BodyScale = BodyScale;
	OutTransform.BodyToGlobal = FTransform(BodyRot, BodyTrans) * GlobalTfm;
}

void FIKRetargetPelvicMotionOp::ComputeTargetBodyTransform(FBodyTransform& OutTransform, FName SourceBoneName, const FTransform& GlobalTfm) const
{
	FName TargetBoneName = ApplyBodyMap(SourceBoneName);
	
	FQuat BodyRot = GetBodyRotation(TargetPhysicsAsset, TargetBoneName);
	FVector BodyTrans = GetBodyTranslation(TargetPhysicsAsset, TargetBoneName);
	// TODO: Probably should always use target body scale
	FVector BodyScale = GetBodyOrientedScale(TargetPhysicsAsset, TargetBoneName);
	
	OutTransform.BoneToBody = GetTargetBoneDelta(TargetBoneName) * FTransform(BodyRot.Inverse());
	OutTransform.BodyScale = BodyScale;
	OutTransform.BodyToGlobal = FTransform(BodyRot, BodyTrans) * GlobalTfm;
}

FVector FIKRetargetPelvicMotionOp::ApplyBodyTransform(const FBodyTransform& Transform, const FVector& LocalPos)
{
	const FTransform& BodyLocalTfm = Transform.BoneToBody;
	const FVector& LocalScale = Transform.BodyScale;
	const FTransform& GlobalTfm = Transform.BodyToGlobal;
	
	return GlobalTfm.TransformPosition(BodyLocalTfm.TransformPositionNoScale(LocalPos) * LocalScale);
}

FVector FIKRetargetPelvicMotionOp::InverseBodyTransform(const FBodyTransform& Transform, const FVector& GlobalPos)
{
	const FTransform& BodyLocalTfm = Transform.BoneToBody;
	const FVector& LocalScale = Transform.BodyScale;
	const FTransform& GlobalTfm = Transform.BodyToGlobal;

	return BodyLocalTfm.InverseTransformPositionNoScale(GlobalTfm.InverseTransformPosition(GlobalPos) / LocalScale);
}

TConstArrayView<FVector3f> FIKRetargetPelvicMotionOp::ApplyTemporalSmoothing(float Time, float SampleRate, int32 NumSamples, int32 NumBodies, const TArray<FVector3f>& PairLocalVerts)
{
	if (NumSamples <= 0)
	{
		return {};
	}
	
	int32 SampleIdx = FMath::Clamp(static_cast<int32>(Time * SampleRate), 0, NumSamples-1);

	// Testing temporal smoothing of local points
	const int32 MinSmoothIdx = FMath::Max(SampleIdx - Settings.TemporalSmoothingRadius, 0);
	const int32 MaxSmoothIdx = FMath::Min(SampleIdx + Settings.TemporalSmoothingRadius,NumSamples-1) ;
	
	if (Settings.TemporalSmoothingRadius <= 0)
	{
		int32 SampleIdx2 = FMath::Clamp(SampleIdx+1, 0, NumSamples-1);
		if (SampleIdx == SampleIdx2)
		{
			return {PairLocalVerts.GetData() + NumBodies * SampleIdx, NumBodies};
		}
		
		float lambda = static_cast<float>(SampleIdx2) - Time * SampleRate;
		SmoothedPoints.SetNumUninitialized(NumBodies);
		for (int BodyIdx = 0; BodyIdx < NumBodies; ++BodyIdx)
		{
			SmoothedPoints[BodyIdx] = lambda * PairLocalVerts[NumBodies * SampleIdx + BodyIdx] + (1.f-lambda) * PairLocalVerts[NumBodies * SampleIdx2 + BodyIdx];
		}
		return SmoothedPoints;
	}
	
	SmoothedPoints.SetNumUninitialized(NumBodies);
	for (int BodyIdx = 0; BodyIdx < NumBodies; ++BodyIdx)
	{
		SmoothedPoints[BodyIdx] = FVector3f::Zero();
		float SumWeight = 0.0f;
		for (int32 FrameIdx = MinSmoothIdx; FrameIdx <= MaxSmoothIdx; ++FrameIdx)
		{
			// Finite support Gaussian smoothing (could approximate w/ triangle or bake out smoothed data)
			const float Sigma = 0.25f * static_cast<float>(Settings.TemporalSmoothingRadius);
			const float SmoothDist = static_cast<float>(FrameIdx) - Time * SampleRate;
			const float SmoothWeight = FMath::Exp(-0.5f*(SmoothDist*SmoothDist) / (Sigma*Sigma));
			SmoothedPoints[BodyIdx] += SmoothWeight * PairLocalVerts[NumBodies * FrameIdx + BodyIdx];
			SumWeight += SmoothWeight;
		}
		SmoothedPoints[BodyIdx] /= SumWeight;
	}
	return SmoothedPoints;
}

FVector FIKRetargetPelvicMotionOp::CalcReferenceShapeScale3D(FKShapeElem* ShapeElem)
{
	FVector ElementScale3D = FVector::OneVector;
	switch (ShapeElem->GetShapeType())
	{
	case EAggCollisionShape::Sphere:
		{
			FKSphereElem* SphereElem = static_cast<FKSphereElem*>(ShapeElem);
			ElementScale3D = FVector(SphereElem->Radius);
			break;
		}

	case EAggCollisionShape::Box:
		{
			FKBoxElem* BoxElem = static_cast<FKBoxElem*>(ShapeElem);
			ElementScale3D = FVector(BoxElem->X, BoxElem->Y, BoxElem->Z);
			break;
		}

	case EAggCollisionShape::Sphyl:
		{
			FKSphylElem* CapsuleElem = static_cast<FKSphylElem*>(ShapeElem);
			ElementScale3D = FVector(CapsuleElem->Radius,CapsuleElem->Radius,CapsuleElem->Length * 0.5f + CapsuleElem->Radius);
			break;
		}

	case EAggCollisionShape::Convex:
		{
			// UE_LOGF(LogAnimation, Warning, "Unsupported: Shape is a Convex");
			break;
		}
	default:
		{
			// UE_LOGF(LogAnimation, Warning, "Unknown or unsupported shape type");
			break;
		}
	}
	
	ensure(!ElementScale3D.IsNearlyZero());
	
	return ElementScale3D;
}

FQuat FIKRetargetPelvicMotionOp::GetBodyRotation(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	FKShapeElem* ShapeElem = FindBodyShape(PhysAsset, BoneName);
	if (!ShapeElem)
	{
		return FQuat::Identity;
	}
	
	FTransform BodyTransform = ShapeElem->GetTransform();
	return BodyTransform.GetRotation();
}

FVector FIKRetargetPelvicMotionOp::GetBodyTranslation(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	FKShapeElem* ShapeElem = FindBodyShape(PhysAsset, BoneName);
	if (!ShapeElem)
	{
		return FVector::ZeroVector;
	}
	
	FTransform BodyTransform = ShapeElem->GetTransform();
	return BodyTransform.GetTranslation();
}

FVector FIKRetargetPelvicMotionOp::GetBodyOrientedScale(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	FKShapeElem* ShapeElem = FindBodyShape(PhysAsset, BoneName);
	if (!ShapeElem)
	{
		return FVector::OneVector;
	}
	
	return CalcReferenceShapeScale3D(ShapeElem);
}

FKShapeElem* FIKRetargetPelvicMotionOp::FindBodyShape(const UPhysicsAsset* PhysAsset, FName BoneName)
{
	int32 BodyIdx = PhysAsset->FindBodyIndex(BoneName);
	if (BodyIdx == INDEX_NONE)
	{
		return nullptr;
	}
	return PhysAsset->SkeletalBodySetups[BodyIdx]->AggGeom.GetElement(0);
}

double FIKRetargetPelvicMotionOp::GetDistanceWeight(EDistanceWeightMode DistanceWeightMode, double Distance, double DistThreshold, double DistHalfPointLambda)
{
	switch (DistanceWeightMode)
	{
	case EDistanceWeightMode::Linear:
		return RationalSmoothStepdown<1>(Distance, DistThreshold, DistHalfPointLambda);
	case EDistanceWeightMode::Quadratic:
		return RationalSmoothStepdown<2>(Distance, DistThreshold, DistHalfPointLambda);
	case EDistanceWeightMode::Cubic:
		return RationalSmoothStepdown<3>(Distance, DistThreshold, DistHalfPointLambda);
	case EDistanceWeightMode::Quartic:
		return RationalSmoothStepdown<4>(Distance, DistThreshold, DistHalfPointLambda);
	default: 
		if (Distance > DistThreshold)
		{
			return 0.;
		}
		return 1.;
	}
	
}

void FIKRetargetPelvicMotionOp::AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent)
{
	if (!IsEnabled() || !bIsInitialized)
	{
		return;
	}
	
	const FAnimMontageInstance* PrevMontageInstance = MontageInstance;
	MontageInstance = nullptr;

	UAnimInstance* SourceAnimInstance = SourceMeshComponent.GetAnimInstance();
	if (!SourceAnimInstance)
	{
		return;
	}
	
	MontageInstance = SourceAnimInstance->GetActiveMontageInstance();
	if (MontageInstance)
	{
		// Early-out for montages
		if (MontageInstance == PrevMontageInstance)
		{
			PreUpdateMontagePlayhead();
			return;
		}

		ResetCacheNotifyInfo();
		SetupRelativeIKNotifyInfoMontage();
		return;
	}

	UpdateRelativeIKNotifyInfoAnimSeq(SourceAnimInstance);
}

void FIKRetargetPelvicMotionOp::UpdateSourceBoneMap(FName SourceBoneName)
{
	int32 SourceIdx = SourceBoneNames.Find(SourceBoneName);
	if (SourceIdx == INDEX_NONE)
	{
		CacheSourceSkelIndices.Remove(SourceBoneName);
		return;
	}

	CacheSourceSkelIndices.Emplace(SourceBoneName, SourceIdx);
}

void FIKRetargetPelvicMotionOp::UpdateTargetBoneMap(FName TargetBoneName)
{
	int32 TargetIdx = TargetBoneNames.Find(TargetBoneName);
	if (TargetIdx == INDEX_NONE)
	{
		CacheTargetSkelIndices.Remove(TargetBoneName);
		return;
	}
	
	CacheTargetSkelIndices.Emplace(TargetBoneName, TargetIdx);
}

void FIKRetargetPelvicMotionOp::UpdateBoneMapTfm(FName SourceBoneName, FName TargetBoneName)
{
	int32 SourceIdx = SourceBoneNames.Find(SourceBoneName);
	int32 TargetIdx = TargetBoneNames.Find(TargetBoneName);
	if (SourceIdx == INDEX_NONE || TargetIdx == INDEX_NONE)
	{
		CacheMapBoneSourceTargetTfm.Remove(TargetBoneName);
		return;
	}

	FQuat SourceRot = CacheSourceBoneInitTfm[SourceIdx].GetRotation();
	FQuat TargetInvRot = CacheTargetBoneInitTfm[TargetIdx].GetRotation().Inverse();

	FTransform TargetDelta = FTransform(TargetInvRot*SourceRot);
	CacheMapBoneSourceTargetTfm.Emplace(TargetBoneName,TargetDelta);
}

void FIKRetargetPelvicMotionOp::UpdateCacheSkelInfo(const FRetargetSkeleton& InSourceSkeleton, const FTargetSkeleton& InTargetSkeleton)
{
	SourceBoneNames = InSourceSkeleton.BoneNames;
	TargetBoneNames = InTargetSkeleton.BoneNames;

	CacheSourceBoneInitTfm = InSourceSkeleton.RetargetPoses.GetGlobalRetargetPose();
	CacheTargetBoneInitTfm = InTargetSkeleton.RetargetPoses.GetGlobalRetargetPose();

	// TODO: Cache less data/directly bake source bone indices and use retarget chains instead of bone mapping
	CacheSourceSkelIndices.Reset();
	CacheTargetSkelIndices.Reset();
}

void FIKRetargetPelvicMotionOp::PreUpdateMontagePlayhead()
{
	if (!MontageInstance)
	{
		AnimSeqPlayHead = -1.0f;
		return;
	}

	// TODO: DeltaTimeRecord doesn't seem to work for montages when scrubbing
	float MontageTime = MontageInstance->GetPosition();
	AnimSeqPlayHead = MontageTime - SegmentStartTime;
	if (MontageTime > SegmentEndTime)
	{
		AnimSeqPlayHead = -1.0f;
	}
}

void FIKRetargetPelvicMotionOp::SetupRelativeIKNotifyInfoMontage()
{
	CacheSourceAnimSequence = nullptr;
	UAnimMontage* Montage = MontageInstance->Montage;
	if (!Montage)
	{
		return;
	}
	
	for (const FSlotAnimationTrack& SlotTrack : Montage->SlotAnimTracks)
	{
		for (const FAnimSegment& Segment : SlotTrack.AnimTrack.AnimSegments)
		{
			const TObjectPtr<UAnimSequenceBase>& SeqRef = Segment.GetAnimReference();
			CacheSourceAnimSequence = Cast<UAnimSequence>(SeqRef.Get());
			if (!CacheSourceAnimSequence)
			{
				continue;
			}

			// TODO: This loop only really works w/ at most one of each notify
			for (const FAnimNotifyEvent& NotifyEvent : SeqRef->Notifies)
			{
				if (URelativePropsBakeAnimNotify* PropInfo = Cast<URelativePropsBakeAnimNotify>(NotifyEvent.Notify))
				{
					SegmentStartTime = Segment.AnimStartTime;
					SegmentEndTime = Segment.AnimEndTime;
					PreUpdateMontagePlayhead();
					UpdateCachePropNotifyInfo(PropInfo);
					break;
				}
			}
		}
	}
}

void FIKRetargetPelvicMotionOp::UpdateRelativeIKNotifyInfoAnimSeq(UAnimInstance* SourceAnimInstance)
{
	// Get source tick records
	TMap<FName, FAnimGroupInstance> SyncGroupTickRecords = SourceAnimInstance->GetSyncGroupMapRead();
	TArray<FAnimTickRecord> UngroupedTickRecords = SourceAnimInstance->GetUngroupedActivePlayersRead();

	// Check for active players with baked notify data, keep hightest blend amount
	float MaxBlend = -1.0f;
	const FAnimTickRecord* MaxNotifyRecord = nullptr;
	auto ProcessTickRecords = [&MaxBlend,&MaxNotifyRecord](const TArray<FAnimTickRecord>& ActivePlayers)
		{
			for (const FAnimTickRecord& ActivePlayer : ActivePlayers)
			{
				if ( ActivePlayer.EffectiveBlendWeight <= MaxBlend )
				{
					continue;
				}
				if (UAnimSequence* Sequence = Cast<UAnimSequence>(ActivePlayer.SourceAsset))
				{
					// Used to determine which notifies should we considered "active".
					check(ActivePlayer.DeltaTimeRecord->IsPreviousValid())
					for (const FAnimNotifyEvent& NotifyEvent : Sequence->Notifies)
					{
						if (URelativePropsBakeAnimNotify* PropInfo = Cast<URelativePropsBakeAnimNotify>(NotifyEvent.Notify))
						{
							MaxBlend = ActivePlayer.EffectiveBlendWeight;
							MaxNotifyRecord = &ActivePlayer;
						}
					}
				}
			}
		};
	
	// Find all the anim sequences (with sync groups) we updated this tick.
	for (const TTuple<FName, FAnimGroupInstance>& SyncGroupPair : SyncGroupTickRecords)
	{
		ProcessTickRecords(SyncGroupPair.Value.ActivePlayers);
	}
	
	// Find all the anim sequences (no sync groups) we updated this tick.
	ProcessTickRecords(UngroupedTickRecords);
	
	if (!MaxNotifyRecord)
	{
		CacheSourceAnimSequence = nullptr;
		ResetCacheNotifyInfo();
		return;
	}

	AnimSeqPlayHead = MaxNotifyRecord->DeltaTimeRecord->GetPrevious() + MaxNotifyRecord->DeltaTimeRecord->Delta;
	if (MaxNotifyRecord->SourceAsset == CacheSourceAnimSequence)
	{
		return;
	}

	ResetCacheNotifyInfo();
	CacheSourceAnimSequence = Cast<UAnimSequence>(MaxNotifyRecord->SourceAsset);

	// TODO: This loop only really works w/ at most one of each notify
	for (const FAnimNotifyEvent& NotifyEvent : CacheSourceAnimSequence->Notifies)
	{
		if (URelativePropsBakeAnimNotify* PropInfo = Cast<URelativePropsBakeAnimNotify>(NotifyEvent.Notify))
		{
			UpdateCachePropNotifyInfo(PropInfo);
			break;
		}
	}
}

void FIKRetargetPelvicMotionOp::ResetCacheNotifyInfo()
{
	SegmentStartTime = -1.0f;
	SegmentEndTime = -1.0f;
	// CachedNotifyInfo = nullptr;
	CachedPropNotifyInfo = nullptr;
	CacheMeshPairOffset = 0;
	CachePropPairOffset = 0;

	// TODO: Need to bake info in accessible bipartite graph structure and cache less data!
	// Clear all cached data on update
	CacheMapBoneSourceTargetTfm.Reset();
	CacheSourceBodyEffectVertIdx.Reset();
	CacheSourceEffectBones.Reset();
	
	CacheBodyBones.Reset();
	CachePairEffectBoneNames.Reset();
	FrameBoneConstraints.Reset();
	
	CachePinSourcePropBones.Reset();
}

void FIKRetargetPelvicMotionOp::UpdateCachePropNotifyInfo(URelativePropsBakeAnimNotify* NotifyInfo)
{
	if (NotifyInfo->bPropsPairsIsParentDominates.IsEmpty())
	{
		return;
	}
	
	if (!CachedPropNotifyInfo)
	{
		CachedPropNotifyInfo = NotifyInfo;
	}
	else
	{
		return;
	}
	CachePropPairOffset = FrameBoneConstraints.Num() / 2;

	int32 NumBodies = CachedPropNotifyInfo->PropsPairs.Num();
	FrameBoneConstraints.AddDefaulted(NumBodies);
	CachePairEffectBoneNames.AddDefaulted(NumBodies);

	PropSampleRate = 30.0f;
	if (CachedPropNotifyInfo->PropsPairsSampleTime.Num() >= 2)
	{
		const float DeltaT = CachedPropNotifyInfo->PropsPairsSampleTime[1] - CachedPropNotifyInfo->PropsPairsSampleTime[0];
		PropSampleRate = FMath::RoundToFloat(1.0f / DeltaT);
	}
	
	// HACK: Only adding pelvis to effect list (we can't move anything else anyway)
	CacheSourceEffectBones.Add(SourcePelvicBone);
	CacheSourceBodyEffectVertIdx.AddDefaulted();
	
	for (const TPair<FName, FTransform>& BodyTfmPair : CachedPropNotifyInfo->OffsetTransformsForBones)
	{
		FName SourceBodyBone = BodyTfmPair.Key;
		FName TargetBodyBone = ApplyBodyMap(SourceBodyBone);
		
		UpdateSourceBoneMap(SourceBodyBone);
		UpdateTargetBoneMap(TargetBodyBone);
		UpdateBoneMapTfm(SourceBodyBone, TargetBodyBone);
	}

	// Cache bone-bone effectors for creating weighted goals
	for (int PairIdx = 0; PairIdx < CachedPropNotifyInfo->bPropsPairsIsParentDominates.Num(); ++PairIdx)
	{
		int32 OutPairIdx = PairIdx + CachePropPairOffset;
		bool bBodyFixed = CachedPropNotifyInfo->bPropsPairsIsParentDominates[PairIdx];
		
		FName PropBone = CachedPropNotifyInfo->PropsPairs[2*PairIdx];
		FName BodyBone = CachedPropNotifyInfo->PropsPairs[2*PairIdx + 1];
		
		if (!KeepPropPair(BodyBone, PropBone))
		{
			continue;
		}
		
		int32 BodyEffectIdx = CacheSourceEffectBones.Find(SourcePelvicBone);
		if (BodyEffectIdx == INDEX_NONE)
		{
			continue;
		}
		
		CacheSourceBodyEffectVertIdx[BodyEffectIdx].Add(2*OutPairIdx+1);

		// Keep track of prop attach bones for post-solve revert
		CachePinSourcePropBones.AddUnique(PropBone);
	}
}

bool FIKRetargetPelvicMotionOp::KeepPropPair(FName SourceBodyBone, FName SourcePropBone)
{
	// // TODO: Can technically remove this to allow other pairs to affect pelvis offset
	// // Only add if it applies to the pelvic bone
	// if (SourceBodyBone != SourcePelvicBone)
	// {
	// 	return false;
	// }
	
	auto PredPinToPropBone = [SourcePropBone](const FPinBoneSettings& Pin)
		{
			return (Pin.SourceBoneName == SourcePropBone);
		};
	
	bool bInCheckAttachList = Settings.CheckPropBones.FindByPredicate(PredPinToPropBone) != nullptr;
	
	// Only add if this pair's prop bone is in the check attach bone list
	return bInCheckAttachList;
}

FIKRetargetOpSettingsBase* FIKRetargetPelvicMotionOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetPelvicMotionOp::GetSettingsType() const
{
	return FIKRetargetPelvicMotionOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetPelvicMotionOp::GetType() const
{
	return FIKRetargetPelvicMotionOp::StaticStruct();
}

#if WITH_EDITOR

FCriticalSection FIKRetargetPelvicMotionOp::DebugDataMutex;

void FIKRetargetPelvicMotionOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	if (!Settings.bDebugDraw)
	{
		return;
	}
	
	FScopeLock ScopeLock(&DebugDataMutex);
	if (Settings.bDebugDrawBodyPairs)
	{
		// Draw Source body pairs
		DebugDrawBodyPairs(InPDI, InSourceTransform, 1.0, DebugDrawInfo.SourcePairVerts);

		// Draw target body pairs (Possibly with "full" retarget space display)
		DebugDrawBodyPairs(InPDI, InComponentTransform, InComponentScale, DebugDrawInfo.TargetPairVerts);
	}
	
	if (Settings.bDebugDrawPhysicsBodies)
	{
		if (Settings.TargetPhysicsAssetOverride)
		{
			// TODO: if only bake source transforms instead of PA, this source draw is meaningless
			DebugDrawBodies(InPDI, InSourceTransform, 1.0, Settings.TargetPhysicsAssetOverride, DebugDrawInfo.SourceBodyInfo, true);
			DebugDrawBodies(InPDI, InComponentTransform, InComponentScale, Settings.TargetPhysicsAssetOverride, DebugDrawInfo.TargetBodyInfo, false);
		}
	}
	
	if (Settings.bDebugDrawBodyTransforms)
	{
		DebugDrawBodyCoords(InPDI, InSourceTransform, 1.0, DebugDrawInfo.SourceTfmInfo);
		DebugDrawBodyCoords(InPDI, InComponentTransform, InComponentScale,  DebugDrawInfo.TargetTfmInfo);
	}

	if (Settings.bDebugDrawRetargetVertAverages)
	{
		// Show retarget contribution average with offsets
		DebugDrawPairVertRetarget(InPDI, InComponentTransform, InComponentScale, DebugDrawInfo.PairRetargetInfo);
	}

	if (Settings.bDebugDrawGoalContributions)
	{
		// Show IK Goals (and contribs)
		DebugDrawGoalContributions(InPDI, InComponentTransform, InComponentScale, DebugDrawInfo.TargetGoals);
	}
}

void FIKRetargetPelvicMotionOp::DebugDrawBodyPairs(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugDrawBodyPair>& BodyPairs) const
{
	const float MaxScale = 1.0f;
	const float MarkerSize = FMath::Min(MaxScale, static_cast<float>(Scale));
	for (const FDebugDrawBodyPair& Pair : BodyPairs)
	{
		float Scalar = FMath::Clamp((float)Pair.Weight, 0.f, 1.f);
		if (FMath::IsNearlyZero(Scalar))
		{
			continue;
		}
		FTransform TfmBodyA_A = FTransform(Pair.PosA_A) * BaseTransform;
		FTransform TfmBodyB_B = FTransform(Pair.PosB_B) * BaseTransform;

		IKRigDebugRendering::DrawWireCube(InPDI, TfmBodyA_A, FLinearColor::Blue*Scalar,MarkerSize,MarkerSize);
		IKRigDebugRendering::DrawWireCube(InPDI, TfmBodyB_B, FLinearColor::Red*Scalar,MarkerSize,MarkerSize);

		// Possibly draw full retarget debug info
		if (Pair.bFullPos)
		{
			FTransform TfmBodyA_B = FTransform(Pair.PosA_B) * BaseTransform;
			FTransform TfmBodyB_A = FTransform(Pair.PosB_A) * BaseTransform;

			IKRigDebugRendering::DrawWireCube(InPDI, TfmBodyA_B, FLinearColor::Black,MarkerSize,MarkerSize);
			IKRigDebugRendering::DrawWireCube(InPDI, TfmBodyB_A, FLinearColor::Yellow,MarkerSize,MarkerSize);

			// Draw direct relationships between bodies in retargeted A/B spaces
			DrawDashedLine(InPDI, TfmBodyA_A.GetLocation(), TfmBodyB_A.GetLocation(), FLinearColor::Black, 1, SDPG_Foreground);
			DrawDashedLine(InPDI, TfmBodyA_B.GetLocation(), TfmBodyB_B.GetLocation(), FLinearColor::Black, 1, SDPG_Foreground);

			// Yellow line showing difference between body A in A-retarget/B-retarget
			DrawDashedLine(InPDI, TfmBodyA_A.GetLocation(), TfmBodyA_B.GetLocation(), FLinearColor::Yellow, 1, SDPG_Foreground);
			// Red line showing difference between body B in A-retarget/B-retarget
			DrawDashedLine(InPDI, TfmBodyB_A.GetLocation(), TfmBodyB_B.GetLocation(), FLinearColor::Red, 1, SDPG_Foreground);
		}
		else
		{
			DrawDashedLine(InPDI, TfmBodyA_A.GetLocation(), TfmBodyB_B.GetLocation(), FLinearColor::Black, 1, SDPG_Foreground);
		}
	}
}

void FIKRetargetPelvicMotionOp::DebugDrawGoalContributions(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugDrawTargetGoal>& GoalInfoLis) const
{
	// Show IK Goals (and contribs)
	const float MaxScale = 1.0f;
	const float MarkerSize = FMath::Min(MaxScale, static_cast<float>(Scale));
	for (const FDebugDrawTargetGoal& Goal : GoalInfoLis)
	{
		FTransform GoalTfm = FTransform(Goal.Goal) * BaseTransform;
		FVector GoalPoint = GoalTfm.GetTranslation();
		for (const FVector& v: Goal.PairTargets)
		{
			FTransform ContribTfm = FTransform(v) * BaseTransform;
			FVector ContribPt = ContribTfm.GetTranslation();
			DrawDashedLine(InPDI, ContribPt, GoalPoint, FLinearColor::Red, 1, SDPG_Foreground);
			IKRigDebugRendering::DrawWireCube(InPDI, ContribTfm, FLinearColor::White,MarkerSize,MarkerSize);
		}
		IKRigDebugRendering::DrawWireCube(InPDI, GoalTfm, FLinearColor::Yellow,MarkerSize,MarkerSize);
	}
}

void FIKRetargetPelvicMotionOp::DebugDrawPairVertRetarget(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugRelativeTargetPairSpace>& RetargetPairList) const
{
	const float MaxScale = 1.0f;
	const float MarkerSize = 0.5f * FMath::Min(MaxScale, static_cast<float>(Scale));
	for (const FDebugRelativeTargetPairSpace& Info : RetargetPairList)
	{
		FTransform V1Tfm = FTransform(Info.PairRangeStart) * BaseTransform;
		FTransform V2Tfm = FTransform(Info.PairRangeEnd) * BaseTransform;
		IKRigDebugRendering::DrawWireCube(InPDI, V1Tfm, FLinearColor::Red,MarkerSize,MarkerSize);
		IKRigDebugRendering::DrawWireCube(InPDI, V2Tfm, FLinearColor::Red,MarkerSize,MarkerSize);

		FVector V1 = V1Tfm.GetTranslation();
		FVector V2 = V2Tfm.GetTranslation();
		// DrawDashedLine(InPDI, VBone, V1, FLinearColor::Yellow, 1, SDPG_Foreground);
		// DrawDashedLine(InPDI, VBone, V2, FLinearColor::Yellow, 1, SDPG_Foreground);
		DrawDashedLine(InPDI, V1, V2, FLinearColor::Black, 1, SDPG_Foreground);

		FVector PairTargetV = FMath::Lerp(Info.PairRangeStart, Info.PairRangeEnd, Info.TargetAlpha);
		FTransform VAvgTfm = FTransform(PairTargetV) * BaseTransform;
		IKRigDebugRendering::DrawWireCube(InPDI, VAvgTfm, FLinearColor::Gray,MarkerSize,MarkerSize);

		FVector FeasibleMinV =  FMath::Lerp(Info.PairRangeStart, Info.PairRangeEnd, Info.FeasibleRange.Min);
		FTransform VMinTfm = FTransform(FeasibleMinV) * BaseTransform;
		IKRigDebugRendering::DrawWireCube(InPDI, VMinTfm, FLinearColor::Yellow,MarkerSize,MarkerSize);

		FVector FeasibleMaxV =  FMath::Lerp(Info.PairRangeStart, Info.PairRangeEnd, Info.FeasibleRange.Max);
		FTransform VMaxTfm = FTransform(FeasibleMaxV) * BaseTransform;
		IKRigDebugRendering::DrawWireCube(InPDI, VMaxTfm, FLinearColor::Green,MarkerSize,MarkerSize);
		// DrawDashedLine(InPDI, VOffset, VOffset + t.Get<3>(), FLinearColor::Black, 1, SDPG_Foreground);
	}
}

void FIKRetargetPelvicMotionOp::DebugDrawBodies(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, UPhysicsAsset* PhysAsset, const TArray<FDebugDrawBodyInfo>& PhysBodies, bool TargetPAForSource) const
{
	for (const FDebugDrawBodyInfo& BodyInfo : PhysBodies)
	{
		FName TargetBodyName = TargetPAForSource? ApplyBodyMap(BodyInfo.BodyName) : BodyInfo.BodyName;
		FKShapeElem* ShapeElem = FindBodyShape(PhysAsset, TargetBodyName);
		if (!ShapeElem)
		{
			continue;
		}
		
		FTransform CompTfm = BodyInfo.BoneTfm * BaseTransform;
		if (TargetPAForSource)
		{
			FTransform TargetShapeTfm = ShapeElem->GetTransform();
			FTransform SourceShapeTfm = BodyInfo.RetargetTfm;
			SourceShapeTfm.RemoveScaling();
			
			CompTfm = TargetShapeTfm.Inverse() * SourceShapeTfm * CompTfm;
			CompTfm.SetScale3D(BodyInfo.RetargetTfm.GetScale3D()/CalcReferenceShapeScale3D(ShapeElem));
		}

		// Draw physics body
		if (DebugDrawInfo.TargetDomainBones.Contains(TargetBodyName))
		{
			DebugDrawPhysBody(InPDI, CompTfm, Scale * BodyInfo.Scale, ShapeElem, FLinearColor::Black);
		}
		else
		{
			DebugDrawPhysBody(InPDI, CompTfm, Scale * BodyInfo.Scale, ShapeElem, FLinearColor::Gray);
		}
	}
}

void FIKRetargetPelvicMotionOp::DebugDrawPhysBody(FPrimitiveDrawInterface* InPDI, const FTransform& ParentTransform, double Scale, const FKShapeElem* ShapeElem, const FLinearColor& Color) const
{
	FVector AbsBodyScale = (ParentTransform.GetScale3D() * Scale).GetAbs();
	FTransform ParentTransformNoScale(ParentTransform);
	ParentTransformNoScale.RemoveScaling();
	FTransform BodyFrame = ShapeElem->GetTransform() * ParentTransformNoScale;

	const FVector Translation = BodyFrame.GetLocation();
	const FVector UnitXAxis = BodyFrame.GetUnitAxis( EAxis::X );
	const FVector UnitYAxis = BodyFrame.GetUnitAxis( EAxis::Y );
	const FVector UnitZAxis = BodyFrame.GetUnitAxis( EAxis::Z );

	
	switch (ShapeElem->GetShapeType())
	{
	case EAggCollisionShape::Box:
		{
			const FKBoxElem* BoxElem = static_cast<const FKBoxElem*>(ShapeElem);
			const FVector Extent = 0.5 * FVector(BoxElem->X, BoxElem->Y, BoxElem->Z) * AbsBodyScale;
			DrawOrientedWireBox(InPDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, Extent, Color, SDPG_Foreground);
			return;
		}
	case EAggCollisionShape::Sphyl:
		{
			const FKSphylElem* CapsuleElem = static_cast<const FKSphylElem*>(ShapeElem);
			const double Radius =CapsuleElem->GetScaledRadius(AbsBodyScale);
			const double HalfHeight = CapsuleElem->GetScaledHalfLength(AbsBodyScale);
			DrawWireCapsule(InPDI, Translation, UnitXAxis, UnitYAxis, UnitZAxis, Color,
						Radius, HalfHeight,25, SDPG_Foreground, 0, 1.0);
			return;
		}
	case EAggCollisionShape::Sphere:
		{
			const FKSphereElem* SphereElem = static_cast<const FKSphereElem*>(ShapeElem);
			const double Radius = SphereElem->Radius * AbsBodyScale.GetMin();
			DrawWireSphere(InPDI, Translation, Color, Radius, 25, SDPG_Foreground);
			return;
		}
	default:
		return;
	}
}


void FIKRetargetPelvicMotionOp::DebugDrawBodyCoords(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugBodyTfmInfo>& BodyTfms) const
{
	// Stop scale from making coords too large
	const float MaxScale = 1.0f;
	const float CoordSize = 10.0f * FMath::Min(MaxScale, static_cast<float>(Scale));
	const float CoordThickness = 0.5f * FMath::Min(MaxScale, static_cast<float>(Scale));

	for (const FDebugBodyTfmInfo& BodyTfm : BodyTfms)
	{

		FVector StartV = BaseTransform.TransformPosition(BodyTfm.Center);
		FVector CoordX = BaseTransform.TransformPosition(BodyTfm.TfmX);
		FVector CoordY = BaseTransform.TransformPosition(BodyTfm.TfmY);
		FVector CoordZ = BaseTransform.TransformPosition(BodyTfm.TfmZ);
		
		InPDI->DrawLine(StartV, CoordX, FLinearColor::Red, SDPG_Foreground, CoordThickness);
		InPDI->DrawLine(StartV, CoordY, FLinearColor::Green, SDPG_Foreground, CoordThickness);
		InPDI->DrawLine(StartV, CoordZ, FLinearColor::Blue, SDPG_Foreground, CoordThickness);
	}
}

void FIKRetargetPelvicMotionOp::UpdateDebugInfo(
	double SourceScale,
	const TArray<FTransform>& SourcePose,
	const TArray<FTransform>& TargetPose,
	const FDebugRelativeIKDrawInfo& LocalDebugInfo)
{
	if (!Settings.bDebugDraw)
	{
		return;
	}
	
	FScopeLock ScopeLock(&DebugDataMutex);
	
	DebugDrawInfo.TargetDomainBones = LocalDebugInfo.TargetDomainBones;

	DebugDrawInfo.TargetGoals = LocalDebugInfo.TargetGoals;
	DebugDrawInfo.PairRetargetInfo = LocalDebugInfo.PairRetargetInfo;
	DebugDrawInfo.SourcePairVerts = LocalDebugInfo.SourcePairVerts;
	DebugDrawInfo.TargetPairVerts = LocalDebugInfo.TargetPairVerts;

	// Cache debug info for source bodies
	DebugDrawInfo.SourceBodyInfo.Reset(CacheSourceEffectBones.Num());
	DebugDrawInfo.TargetBodyInfo.Reset(CacheSourceEffectBones.Num());
	for (int SourceEffectBoneIndex = 0; SourceEffectBoneIndex<CacheSourceEffectBones.Num(); SourceEffectBoneIndex++)
	{
		FName SourceBoneName = CacheSourceEffectBones[SourceEffectBoneIndex];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		
		if (CacheSourceSkelIndices.Contains(SourceBoneName))
		{
			int32 SourcePoseIdx = CacheSourceSkelIndices[SourceBoneName];
			// Need to apply source-scale to bodies since it's directly applied to translations only in source pose
			FTransform SourceScaleTfm{FQuat::Identity, FVector::ZeroVector, FVector(SourceScale)};
			DebugDrawInfo.SourceBodyInfo.Emplace(SourceBoneName, SourceScale, SourceScaleTfm*SourcePose[SourcePoseIdx], OffsetTransformsForSourceBones(SourceBoneName));
		}
		
		if (CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			int32 TargetPoseIdx = CacheTargetSkelIndices[TargetBoneName];
			DebugDrawInfo.TargetBodyInfo.Emplace(TargetBoneName, 1.0, TargetPose[TargetPoseIdx], FTransform::Identity);
		}
	}

	DebugDrawInfo.SourceTfmInfo.Reset(CacheSourceEffectBones.Num());
	DebugDrawInfo.TargetTfmInfo.Reset(CacheSourceEffectBones.Num());
	for (int SourceEffectBoneIndex = 0; SourceEffectBoneIndex<CacheSourceEffectBones.Num(); SourceEffectBoneIndex++)
	{
		FName SourceBoneName = CacheSourceEffectBones[SourceEffectBoneIndex];
		FName TargetBoneName = ApplyBodyMap(SourceBoneName);
		
		if (CacheSourceSkelIndices.Contains(SourceBoneName))
		{
			int32 SourcePoseIdx = CacheSourceSkelIndices[SourceBoneName];
		
			FBodyTransform SourceFullTfm;
			FTransform SourceBodyRot(OffsetTransformsForSourceBones(SourceBoneName).GetRotation());
			ComputeSourceBodyTransform(SourceFullTfm, SourceBoneName, SourcePose[SourcePoseIdx], SourceScale);

			FVector SourceBodyCenter = ApplyBodyTransform(SourceFullTfm, FVector::ZeroVector);
			FVector SourceBodyX = ApplyBodyTransform(SourceFullTfm, SourceBodyRot.GetUnitAxis(EAxis::X));
			FVector SourceBodyY = ApplyBodyTransform(SourceFullTfm, SourceBodyRot.GetUnitAxis(EAxis::Y));
			FVector SourceBodyZ = ApplyBodyTransform(SourceFullTfm, SourceBodyRot.GetUnitAxis(EAxis::Z));
		
			DebugDrawInfo.SourceTfmInfo.Emplace(SourceBodyCenter, SourceBodyX, SourceBodyY, SourceBodyZ);
		}

		if (CacheTargetSkelIndices.Contains(TargetBoneName))
		{
			int32 TargetPoseIdx = CacheTargetSkelIndices[TargetBoneName];

			FBodyTransform TargetFullTfm;
			FTransform SourceBodyRot(OffsetTransformsForSourceBones(SourceBoneName).GetRotation());
			ComputeTargetBodyTransform(TargetFullTfm, SourceBoneName, TargetPose[TargetPoseIdx]);

			FVector TargetBodyCenter = ApplyBodyTransform(TargetFullTfm, FVector::ZeroVector);
			FVector TargetBodyX = ApplyBodyTransform(TargetFullTfm, SourceBodyRot.GetUnitAxis(EAxis::X));
			FVector TargetBodyY = ApplyBodyTransform(TargetFullTfm, SourceBodyRot.GetUnitAxis(EAxis::Y));
			FVector TargetBodyZ = ApplyBodyTransform(TargetFullTfm, SourceBodyRot.GetUnitAxis(EAxis::Z));
		
			DebugDrawInfo.TargetTfmInfo.Emplace(TargetBodyCenter, TargetBodyX, TargetBodyY, TargetBodyZ);
		}
	}
}

void FIKRetargetPelvicMotionOp::ResetDebugInfo()
{
	FScopeLock ScopeLock(&DebugDataMutex);
	
	DebugDrawInfo.SourcePairVerts.Reset();
	DebugDrawInfo.TargetPairVerts.Reset();
	
	DebugDrawInfo.SourceBodyInfo.Reset();
	DebugDrawInfo.TargetBodyInfo.Reset();

	DebugDrawInfo.SourceTfmInfo.Reset();
	DebugDrawInfo.TargetTfmInfo.Reset();
	
	DebugDrawInfo.PairRetargetInfo.Reset();
	DebugDrawInfo.TargetGoals.Reset();
}

#endif //WITH_EDITOR

FIKRetargetPelvicMotionOpSettings UIKRetargetPelvicMotionController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetPelvicMotionOpSettings*>(OpSettingsToControl);
}

void UIKRetargetPelvicMotionController::SetSettings(FIKRetargetPelvicMotionOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE

