// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/RetargetOps/PinBoneOp.h"
#include "Engine/SkeletalMesh.h"
#include "Eigen/Dense"

#include "RelativeIKOp.generated.h"

#define UE_API RELATIVEIKOP_API

#define LOCTEXT_NAMESPACE "RelativeIKOp"

class BODYINTERSECTIKOP_API FPhysShapeUtils;
struct FIKRigGoal;
struct FKShapeElem;
struct FKAggregateGeom;
struct FResolvedBoneChain;
struct FAnimMontageInstance;
struct FIKRetargetRunIKRigOp;
class UAnimInstance;
class UAnimSequence;
class UPhysicsAsset;
class USkeletalMesh;
class URelativeBodyBakeAnimNotify;
class URelativePropsBakeAnimNotify;

UENUM(BlueprintType)
enum class EDistanceWeightMode : uint8
{
	// Constant   UMETA(DisplayName = "Constant"),
	Linear     UMETA(DisplayName = "Linear"),
	Quadratic  UMETA(DisplayName = "Quadratic"),
	Cubic      UMETA(DisplayName = "Cubic"),
	Quartic    UMETA(DisplayName = "Quartic"),
	// Quintic    UMETA(DisplayName = "Quintic")
};

// Debug info for individual body pair contributions
struct FDebugRelativeTargetPairSpace
{
	FVector PairRangeStart;
	FVector PairRangeEnd;

	double TargetAlpha;
	FDoubleInterval FeasibleRange;
};

// Debug draw info for goal (with individual pair goal targets)
struct FDebugDrawTargetGoal
{
	FVector Goal;
	TArray<FVector> PairTargets;
};

// Debug draw info for source or target pair
struct FDebugDrawBodyPair
{
	FVector PosA_A;
	FVector PosB_B;
	
	bool bFullPos;
	FVector PosB_A;
	FVector PosA_B;

	double Weight;
};

// Debug physics body info
struct FDebugDrawBodyInfo
{
	FName BodyName;
	double Scale;
	FTransform BoneTfm;
	FTransform RetargetTfm = FTransform::Identity;
};

struct FDebugBodyTfmInfo
{
	FVector Center;
	FVector TfmX;
	FVector TfmY;
	FVector TfmZ;
};

// Used to hold all debug draw info for convenience
struct FDebugRelativeIKDrawInfo
{
	// Source/target vert pair info
	TArray<FDebugDrawBodyPair> SourcePairVerts;
	TArray<FDebugDrawBodyPair> TargetPairVerts;

	// Body info
	TArray<FDebugDrawBodyInfo> SourceBodyInfo;
	TArray<FDebugDrawBodyInfo> TargetBodyInfo;

	// Body transform vecs
	TArray<FDebugBodyTfmInfo> SourceTfmInfo;
	TArray<FDebugBodyTfmInfo> TargetTfmInfo;

	// Debug info for retarget pairs
	TArray<FDebugRelativeTargetPairSpace> PairRetargetInfo;
	// Final goal points with contributions
	TArray<FDebugDrawTargetGoal> TargetGoals;

	TArray<FName> TargetDomainBones;
};

USTRUCT(BlueprintType)
struct FPinBoneSettings
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pinning", meta=(ReinitializeOnEdit))
	FName SourceBoneName;
	
	// The method used to calculate the translation of the bone.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pinning", meta=(ReinitializeOnEdit))
	EPinBoneTranslationMode TranslationMode = EPinBoneTranslationMode::CopyGlobalPosition;
	
	// The method used to calculate the rotation of the bone.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pinning", meta=(ReinitializeOnEdit))
	EPinBoneRotationMode RotationMode = EPinBoneRotationMode::CopyGlobalRotation;
	
	// Alpha between primary and secondary pair distance relationship
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pinning", meta=(ReinitializeOnEdit))
	double PropScalar = 1.0;
	
	// Prop Skeletal Mesh
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(ReinitializeOnEdit))
	TObjectPtr<USkeletalMesh> PropSkeletalMeshAsset;
	
	// Prop mesh to anim sequence
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(ReinitializeOnEdit))
	TObjectPtr<UAnimSequence> PropAnimSequence;
};

UENUM()
enum class ERelativeIKFrameBoneConstraintType : uint8
{
	XYZ = 0,
	Z = 1,
	Default = XYZ,
};

struct FRelativeIKFrameBoneConstraint
{
	ERelativeIKFrameBoneConstraintType ConstraintType;
	
	double Weight;
	double FeasibilityWeight;
	FVector ContactOffset;
	FVector Offset;
	bool bAmplifyCloseWeight;
	double SourceDist;

	FRelativeIKFrameBoneConstraint()
	{
		Reset();
	}

	void Reset()
	{
		ConstraintType = ERelativeIKFrameBoneConstraintType::Default;
		Weight = 0.0;
		FeasibilityWeight = 0.0;
		Offset = FVector::ZeroVector;
		ContactOffset = FVector::ZeroVector;
		bAmplifyCloseWeight = false;
		SourceDist = 0.0;
	}
};

USTRUCT(BlueprintType, meta = (DisplayName = "Relative IK Settings"))
struct FIKRetargetRelativeIKOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	// Target mesh physics asset for retargeting relative body pair vertices
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta=(ReinitializeOnEdit))
	TObjectPtr<UPhysicsAsset> TargetPhysicsAssetOverride;
	
	// Physics body source -> target name mapping 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source Target Mapping", meta=(ReinitializeOnEdit))
	TMap<FName,FName> BodyMapping;

	// Maximum distance for which prop-floor info is baked
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0.1"))
	double FloorThreshold = 5.00;
	
	// Maximum distance for which prop-floor info is baked
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0.0"))
    double FloorConstraintScalar = 10.00;
	
	// List of bones to pin. Should include all attach bones that may be affected by relative ik bake data 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Settings", meta=(ReinitializeOnEdit))
	TArray<FPinBoneSettings> PinPropBones;
	
	// Maximum distance for which body pair info is baked
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0.1"))
	double DistanceThreshold = 70.00;

	// DistHalfPointLambda*DistanceThreshold is the place where distance weight value be 0.5
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0.01", ClampMax = "0.99"))
	double DistHalfPointLambda = 0.15;

	// DistHalfPointLambda*DistanceThreshold is the place where distance weight value be 0.5
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	EDistanceWeightMode DistanceWeightMode = EDistanceWeightMode::Quadratic;

	// Bias feasibility distance weighting relative to distance threshold (+ increase target feasibility/- reduced target feasibility)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	double FeasibilityLengthBias = 0.0;

	// IK Goal normalization (internally multiplied by total of contribution weights)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0", ClampMax = "1"))
	double ContributionSumWeight = 1.0;

	// Frames of temporal smoothing of body pair verts (0 is no smoothing)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0", ClampMax = "60"))
	int32 TemporalSmoothingRadius = 5;

	// Ignore source scaling when computing relative distance relationships
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	bool bIgnoreSourceScale = true;

	// Alpha from contact-body to secondary body representation in contribution pairs
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	double RetargetContactAlpha = 0.5;

	// Alpha between primary and secondary pair distance relationship
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	double RetargetSpringAlpha = 0.5;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Importance Weight")
	bool bApplyImportanceWeighting = true;
	
	// TODO: Maybe use IKGoal bones (or detail customization to set automatically)
	// Amplify solve weight when "important bones" are close to props
	// UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Importance Weight")
	// TArray<FName> ImportantBones;
	
	// Start amplifying weight at this distance (close-ish to prop)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Importance Weight", meta=(ClampMin = "1.0"))
	double AmplifyCloseThreshold = 15.0;

	// Max weight increase cap distance (very close to prop)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Importance Weight", meta=(ClampMin = "0.1"))
	double AmplifyCloseDistance = 3.0;

	// Bias feasibility distance weighting relative to distance threshold (+ increase target feasibility/- reduced target feasibility)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Importance Weight", meta=(ClampMin = "1.0", ClampMax = "20.0"))
	double AmplifyScale = 6.0;

	// Draw source and target body pair relationships for current animation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawBodyPairs = false;

	// Draw full retarget space (quad) pair relationships
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug", meta=(NotOverrideable))
	TArray<FName> DebugFullRetargetPairBones;

	// Draw each pair's goal contribution (white) and show weighted final goal location (yellow)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawGoalContributions = false;

	// Draw retarget pair-line for each pair contribution 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawRetargetVertAverages = false;

	// Display source and target physics bodies for baked data
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawPhysicsBodies = false;

	// Show body-local transform setup (like baked verts)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawBodyTransforms = false;

	// Run op and display debug info but DON'T update IK Goals
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDryRun = false;

	// // Distance blend contact bodies towards domain bodies
	// UPROPERTY(BlueprintReadWrite, Category = "Test Features")
	// bool bTestDistContactAlpha = true;
	//
	// // Reduce distance contact blending based on feasibility length of target bone chain
	// UPROPERTY(BlueprintReadWrite, Category = "Test Features")
	// bool bTestFeasibilityWeight = true;

	// Solve for output positions using multiple interacting contact bodies
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Test Features")
	bool bMultiBoneSolve = true;
	
	// Regularization additive constant for stabilizing multibone solve, the higher the body bones will stay at original position
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Test Features", meta=(ClampMin = "0.001"))
	double MultiBoneRegularization = 0.001;

		
	// Add Props scale as a parameter for optimization
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Scaling")
	bool bSolveWithPropScale = false;
	
	// Props scalar change ratio which allow to change per frame
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Scaling", meta=(ClampMin = "0.0001", ClampMax = "10"))
	double PropScalarLearningRate = 0.001;
	
	// Test Parameter for prop scalar range allow to change, it will clamp prop scalar between [PinPropBone.Propscalar-PropScalarRange, PinPropBone.Propscalar+PropScalarRange]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Scaling", meta=(ClampMin = "0", ClampMax = "10"))
	double PropScalarRange = 0.25;
	
	// Regularization additive constant for stabilizing prop bone position, the higher the prop bones will stay at original position
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Scaling", meta=(ClampMin = "0.001"))
	double PropScalarBoneRegularization = 1;
	
	// Regularization additive constant for stabilizing prop bone scalar, the higher the prop bones will stay at original Propscalar set in PinPropBone Settings 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Scaling", meta=(ClampMin = "0.001"))
	double PropScalarDamping = 500;

	// Enable Collison push out for Ikgoals and Props
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection")
	bool bEnableIntersectPushOut = true;
	
	// Target mesh physics asset for Intersect push out
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(ReinitializeOnEdit))
	TObjectPtr<UPhysicsAsset> TargetIntersectOpPhysicsAssetOverride;
	
	// Max distance to glue bodies (very close to a body)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(ClampMin = "0.01"))
	double GlueThreshold = 5;
	
	// Max distance to glue bodies (very close to a body)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(ClampMin = "0.01"))
	double IntersectOffsetScale = 1;
	
	// Bone names with attached physics bodies to do trivial intersection against
	// **NOTE: Only Sphere, Capsule and Box bodies currently supported**
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Intersection", meta=(NotOverrideable))
	TArray<FName> IntersectBodies;
	
	// Distance blend body pairs towards each contact position if both can move
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Test Features")
	bool bBidirectionalDistanceContactBlend = true;

	// Apply prop pin post-solve
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Test Features")
	bool bApplyPostSolvePinning = true;
	
	// Blend from relative ik pin bone to source location
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Test Features", meta=(ClampMin = "0", ClampMax = "1"))
	double BlendPinsToSource = 0.0;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Relative IK Goals"))
struct FIKRetargetRelativeIKOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void RunAfterParent(FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent) override;

	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;
	
	UE_API virtual const UScriptStruct* GetParentOpType() const override;

	UE_API virtual void OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent) override;

	UPROPERTY()
	FIKRetargetRelativeIKOpSettings Settings;

private:
	void UpdateAnimPlayhead(float FrameTime);
	void UpdateAnimSeqData(UAnimSequence* Sequence);
	void UpdateCacheBoneChains(const FIKRetargetProcessor& InProcessor, const FIKRetargetRunIKRigOp* ParentRigOp);
	void UpdateSourceBoneMap(FName SourceBoneName);
	void UpdateTargetBoneMap(FName TargetBoneName);
	void UpdateBoneMapTfm(FName SourceBoneName, FName TargetBoneName);
	void UpdateCacheSkelInfo(const FRetargetSkeleton& InSourceSkeleton, const FTargetSkeleton& InTargetSkeleton);
	void PreUpdateMontagePlayhead();
	void SetupRelativeIKNotifyInfoMontage();
	void UpdateRelativeIKNotifyInfoAnimSeq(UAnimInstance* SourceAnimInstance);
	void ResetCacheNotifyInfo();
	void UpdateCacheNotifyInfo(URelativeBodyBakeAnimNotify* NotifyInfo);
	void UpdateCacheChainInfo(const TArray<FTransform>& TargetPose, FName BoneName);
	void UpdateCachePropNotifyInfo(URelativePropsBakeAnimNotify* NotifyInfo);
	
	struct FRelativeIKPinBoneData
	{
		EPinBoneTranslationMode TranslationMode;
		EPinBoneRotationMode RotationMode;
		// TODO: Switch to using bone indices where possible
		FName SourceBone;
		FName TargetBone;
	
		FTransform SourceTargetOffsetTfm;
		FTransform SourceLocalRefTfm;
		FTransform TargetLocalRefTfm;
		FQuat RefRotDelta;
		
		double PropScalar;
	};
	
	void InitializePinData(const FIKRetargetProcessor& Processor);
	void InitializeIntersectionData(const FIKRetargetProcessor& Processor);
	
	void CachePinBoneTransforms(
		const FIKRetargetProcessor& Processor,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose);
	
	FTransform ComputePinBoneTransform(
		const FRelativeIKPinBoneData& PinData,
		const FIKRetargetProcessor& Processor,
		const TArray<FTransform>& InSourceGlobalPose,
		const TArray<FTransform>& OutTargetGlobalPose);
	
	static const FTransform& GetParentTransform(const FRetargetSkeleton& Skeleton, const int32 BoneIndex, const TArray<FTransform>& InPose);
	
	void ComputeFramePairSpaces(
		double SourceScale,
		const TArray<FTransform>& SourcePose,
		const TArray<FTransform>& TargetPose,
		int32 PairOffset,
		TConstArrayView<FVector3f> FramePointView,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);

	void ComputeFramePropSpaces(
		double SourceScale,
		const TArray<FName>& AmplifyBones,
		const TArray<FTransform>& SourcePose,
		const TArray<FTransform>& TargetPose,
		int32 PairOffset,
		TConstArrayView<FVector3f> FramePointView,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);
	
	void AmplifyImportanceWeights();

	void ApplyAverageBodyPairTargets(
		TArray<FIKRigGoal>& OutIKGoals,
		const TArray<FTransform>& TargetPose,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);
	
	void ApplyMatrixBodyPairTargets(
		TArray<FIKRigGoal>& OutIKGoals,
		const TArray<FTransform>& TargetPose,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);
	
	void ApplyIntersectPushOut(
		TArray<FIKRigGoal>& OutIKGoals,
		const TArray<FTransform>& TargetPose,
		FDebugRelativeIKDrawInfo& DebugInfo);
	
	void ApplyMatrixBodyPairTargetsWithPropScale(
		TArray<FIKRigGoal>& OutIKGoals,
		const TArray<FTransform>& TargetPose,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);

	void UpdateTargetIKGoal(
		TArray<FIKRigGoal>& OutIKGoals,
		FName TargetBoneName,
		const FVector& TargetWeight,
		const FVector& TargetGoalOffset,
		const FTransform& InTargetBonePose);

	void UpdatePropPinTransform(
		FName SourceBoneName,
		const FVector& TargetWeight,
		const FVector& TargetPropOffset,
		const FTransform& InTargetBonePose);

	FVector ComputeTargetWeightedOffset(
		FVector& OutTargetOffset,
		const TArray<int32>& EffectIndices,
		const FVector& BoneLoc,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);

	void ComputeWeightMatrixRow(
		Eigen::MatrixXd& WeightMatrix,
		int32 RowIdx,
		const TArray<FName>& OutBones,
		const TArray<int32>& EffectIndices);

	FVector ComputeTargetWeightedOffsetRow(
		Eigen::MatrixXd& OffsetMatrix,
		int32 RowIdx,
		const TArray<FName>& OutBones,
		const TArray<int32>& EffectIndices,
		const FVector& BoneLoc,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);

	FName ApplyBodyMap(FName BodyName) const;
	bool HasBoneDelta(FName TargetBoneName) const;
	const FTransform& GetTargetBoneDelta(FName TargetBoneName) const;

	// Internal structure for holding necessary baked->component transform data
	// Baked data is in Source Bone orientation but at body origin
	struct FBodyTransform
	{
		// TODO: Test this with FMatrix computations and see what's better/faster
		FTransform BoneToBody;
		FVector BodyScale;
		FTransform BodyToGlobal;
	};

	const FTransform& OffsetTransformsForSourceBones(FName SourceBoneName) const;
	void ComputeSourceBodyTransform(FBodyTransform& OutTransform, FName SourceBoneName, const FTransform& GlobalTfm, double SourceScale) const;
	void ComputeTargetBodyTransform(FBodyTransform& OutTransform, FName SourceBoneName, const FTransform& GlobalTfm) const;

	static FVector ApplyBodyTransform(const FBodyTransform& Transform, const FVector& LocalPos);
	static FVector InverseBodyTransform(const FBodyTransform& Transform, const FVector& GlobalPos);
	
	void ComputeFeasibilityRange(FDoubleInterval& OutFeasibleRange, FName BoneName, const FVector& BoneLoc, const FVector& TargetOffsetVec);
	TConstArrayView<FVector3f> ApplyTemporalSmoothing(float Time, float SampleRate, int32 NumSamples, int32 NumBodies, const TArray<FVector3f>& PairLocalVerts);

	FVector GoalLocBlendCompSpace(const FIKRigGoal* Goal, const FTransform& BoneTfm) const;

	static bool CalcLineSphereIntersect(FDoubleInterval& OutRange, const FVector& Center, double Radius, const FVector& StartPoint, const FVector& EndPoint);

	static FVector CalcReferenceShapeScale3D(FKShapeElem* ShapeElem);
	// Get local body rotation relative to parent bone
	static FQuat GetBodyRotation(const UPhysicsAsset* PhysAsset, FName BoneName);
	// Get body origin relative to parent bone
	static FVector GetBodyTranslation(const UPhysicsAsset* PhysAsset, FName BoneName);
	// Get body oriented scale vector
	static FVector GetBodyOrientedScale(const UPhysicsAsset* PhysAsset, FName BoneName);
	static FKShapeElem* FindBodyShape(const UPhysicsAsset* PhysAsset, FName BoneName);
	
	static double GetDistanceWeight(EDistanceWeightMode DistanceWeightMode, double Distance, double DistThreshold, double DistHalfPointLambda = 0.5);
	void GetAnimSeqFramePose(const UAnimSequence* AnimSeq, double Time, TArray<FName>& OutBones, TArray<FTransform>& OutPose) const;

#if WITH_EDITOR
public:
	UE_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;

	virtual bool HasDebugDrawing() override { return true; };
	
	UE_API virtual void UpdateFromAnimSequence(UAnimSequence* Sequence, float FrameTime) override;

private:
	void DebugDrawBodyPairs(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugDrawBodyPair>& BodyPairs) const;
	void DebugDrawGoalContributions(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugDrawTargetGoal>& GoalInfoLis) const;
	void DebugDrawPairVertRetarget(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugRelativeTargetPairSpace>& RetargetPairList) const;
	void DebugDrawBodies(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, UPhysicsAsset* PhysAsset, const TArray<FDebugDrawBodyInfo>& PhysBodies, bool TargetPAForSource) const;
	void DebugDrawPhysBody(FPrimitiveDrawInterface* InPDI, const FTransform& ParentTransform, double Scale, const FKShapeElem* ShapeElem, const FLinearColor& Color) const;
	void DebugDrawBodyCoords(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugBodyTfmInfo>& BodyTfms) const;

	void UpdateDebugInfo(double SourceScale, const TArray<FTransform>& SourcePose, const TArray<FTransform>& TargetPose, const FDebugRelativeIKDrawInfo& LocalDebugInfo);
	void ResetDebugInfo();

	// Collection of all debug structures
	FDebugRelativeIKDrawInfo DebugDrawInfo;
	
	static UE_API FCriticalSection DebugDataMutex;
#endif
private:
	
	// TObjectPtr<const UPhysicsAsset> SourcePhysicsAsset;
	TObjectPtr<const UPhysicsAsset> TargetRelativeIKPhysicsAsset;
	TObjectPtr<const UPhysicsAsset> TargetIntersectOpPhysicsAsset;
	
	TObjectPtr<const UIKRigDefinition> TargetIKRig;
	
	// For runtime smoothing
	TArray<FVector3f> SmoothedPoints;

	// Cache notify-related data for faster weighted goal computations
	TObjectPtr<const UAnimSequence> CacheSourceAnimSequence;
	TObjectPtr<const URelativeBodyBakeAnimNotify> CachedNotifyInfo;
	TObjectPtr<const URelativePropsBakeAnimNotify> CachedPropNotifyInfo;
	int32 CacheMeshPairOffset = 0;
	int32 CachePropPairOffset = 0;
	float AnimSeqPlayHead;
	float MeshSampleRate;
	float PropSampleRate;

	// Cache source/target bone-names
	TArray<FName> SourceBoneNames;
	TArray<FName> TargetBoneNames;

	// Array of body-vert indices affecting each joint goal (for weighted averaging)
	TArray<TArray<int32>> CacheSourceBodyEffectVertIdx;
	TArray<FName> CacheSourceEffectBones;
	TSet<FName> CacheBodyBones;
	TArray<FName> CachePairEffectBoneNames;
	
	TArray<FRelativeIKFrameBoneConstraint> FrameBoneConstraints;

	// Cache skel and IK goal info
	TArray<FTransform> CacheSourceBoneInitTfm;
	TArray<FTransform> CacheTargetBoneInitTfm;
	TMap<FName,FTransform> CacheMapBoneSourceTargetTfm;
	// TMap<FName,FName> CacheIKBoneGoalMap;
	TMap<FName, const FResolvedBoneChain*> CacheBoneChains;
	TMap<FName,int32> CacheSourceSkelIndices;
	TMap<FName,int32> CacheTargetSkelIndices;
	// Additional per-frame ik cache maps for length lookup
	TMap<FName,double> CacheChainLengthMap;
	TMap<FName,FVector> CacheChainStartMap;
	// Cache prop bones to pin/update
	TArray<FName> CachePinSourcePropBones;
	TArray<FRelativeIKPinBoneData> CachePinBoneData;
	TArray<FTransform> CachePinTargetPropTransforms;
	TArray<int32> CachePropId2PinTargetPropSettingId;
	TArray<int32> CachePropId2PinBoneDataId;

	// Cached montage instance/segment info for early-out matching in preupdate
	const FAnimMontageInstance* MontageInstance;
	float SegmentStartTime;
	float SegmentEndTime;
};

/* The blueprint/python API for editing a Relative IK Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetRelativeIKController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current op settings as a struct.
	 * @return FIKRetargetRelativeIKOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetRelativeIKOpSettings GetSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRetargetRelativeIKOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetRelativeIKOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
