// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/RetargetOps/PinBoneOp.h"
#include "Eigen/Dense"

// TODO: Move all of the interdependencies into helper files/class for both ops!!
#include "RelativeIKOp.h"

#include "RelativePelvicMotionOp.generated.h"

#define UE_API RELATIVEIKOP_API

#define LOCTEXT_NAMESPACE "RelativePelvicMotionOp"

struct FKShapeElem;
struct FResolvedBoneChain;
struct FAnimMontageInstance;
struct FIKRetargetRunIKRigOp;
class UAnimInstance;
class UAnimSequence;
class UPhysicsAsset;
class URelativeBodyBakeAnimNotify;
class URelativePropsBakeAnimNotify;

USTRUCT(BlueprintType, meta = (DisplayName = "Relative Pelvic Motion Settings"))
struct FIKRetargetPelvicMotionOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	// Target mesh physics asset for retargeting relative body pair vertices
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Physics", meta=(ReinitializeOnEdit))
	TObjectPtr<UPhysicsAsset> TargetPhysicsAssetOverride;

	// Physics body source -> target name mapping 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source Target Mapping", meta=(ReinitializeOnEdit))
	TMap<FName,FName> BodyMapping;
	
	// Physics body source -> target name mapping 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Target", meta=(ReinitializeOnEdit))
	FName TargetPelvicBone = NAME_None;
	
	// List of attach (prop) bones that can contribute to pelvic motion relationships
	// NOTE: unlike relative ik, these bones are not pinned by the op
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Prop Settings", meta=(ReinitializeOnEdit))
	TArray<FPinBoneSettings> CheckPropBones;
	
	// Maximum distance for which body pair info is baked
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0.1"))
	double DistanceThreshold = 70.00;

	// DistHalfPointLambda*DistanceThreshold is the place where distance weight value be 0.5
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0.01", ClampMax = "0.99"))
	double DistHalfPointLambda = 0.15;

	// DistHalfPointLambda*DistanceThreshold is the place where distance weight value be 0.5
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	EDistanceWeightMode DistanceWeightMode = EDistanceWeightMode::Quadratic;

	// IK Goal normalization (internally multiplied by total of contribution weights)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0", ClampMax = "1"))
	double ContributionSumWeight = 1.0;

	// Frames of temporal smoothing of body pair verts (0 is no smoothing)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0", ClampMax = "60"))
	int32 TemporalSmoothingRadius = 15;

	// Ignore source scaling when computing relative distance relationships
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters")
	bool bIgnoreSourceScale = true;

	// Alpha from contact-body to secondary body representation in contribution pairs
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Parameters", meta=(ClampMin = "0", ClampMax = "1"))
	double RetargetContactAlpha = 0.5;

	// Alpha between primary and secondary pair distance relationship
	UPROPERTY(BlueprintReadWrite, Category = "Parameters")
	double RetargetSpringAlpha = 0.5;

	// Draw source and target body pair relationships for current animation
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawBodyPairs = false;

	// Draw full retarget space (quad) pair relationships
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug", meta=(NotOverrideable))
	TArray<FName> DebugFullRetargetPairBones;

	// Draw each pair's goal contribution (white) and show weighted final goal location (yellow)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawGoalContributions = false;

	// Draw retarget pair-line for each pair contribution 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawRetargetVertAverages = false;

	// Display source and target physics bodies for baked data
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawPhysicsBodies = false;

	// Show body-local transform setup (like baked verts)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDebugDrawBodyTransforms = false;

	// Run op and display debug info but DON'T update IK Goals
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	bool bDryRun = false;
	
	// // Distance blend body pairs towards each contact position if both can move
	// UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Test Features")
	// bool bBidirectionalDistanceContactBlend = false;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Relative Pelvic Motion"))
struct FIKRetargetPelvicMotionOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;

	UPROPERTY()
	FIKRetargetPelvicMotionOpSettings Settings;

private:
	void UpdateSourceBoneMap(FName SourceBoneName);
	void UpdateTargetBoneMap(FName TargetBoneName);
	void UpdateBoneMapTfm(FName SourceBoneName, FName TargetBoneName);
	void UpdateCacheSkelInfo(const FRetargetSkeleton& InSourceSkeleton, const FTargetSkeleton& InTargetSkeleton);
	void PreUpdateMontagePlayhead();
	void SetupRelativeIKNotifyInfoMontage();
	void UpdateRelativeIKNotifyInfoAnimSeq(UAnimInstance* SourceAnimInstance);
	void ResetCacheNotifyInfo();
	// void UpdateCacheNotifyInfo(URelativeBodyBakeAnimNotify* NotifyInfo);
	void UpdateCachePropNotifyInfo(URelativePropsBakeAnimNotify* NotifyInfo);
	bool KeepPropPair(FName SourceBodyBone, FName SourcePropBone);
	
	struct FRelativeIKPinBoneData
	{
		EPinBoneTranslationMode TranslationMode;
		EPinBoneRotationMode RotationMode;
		// TODO: Switch to using bone indices where possible
		FName SourceBone;
		FName TargetBone;
	
		FTransform SourceTargetOffsetTfm;
		FTransform SourceLocalRefTfm;
		FTransform TargetLocalRefTfm;
		FQuat RefRotDelta;
	};
	
	void InitializePinData(const FIKRetargetProcessor& Processor);
	
	void CachePinBoneTransforms(
		const FIKRetargetProcessor& Processor,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose);
	
	FTransform ComputePinBoneTransform(
		const FRelativeIKPinBoneData& PinData,
		const FIKRetargetProcessor& Processor,
		const TArray<FTransform>& InSourceGlobalPose,
		const TArray<FTransform>& OutTargetGlobalPose);
	
	static const FTransform& GetParentTransform(const FRetargetSkeleton& Skeleton, const int32 BoneIndex, const TArray<FTransform>& InPose);

	void ComputeFramePropSpaces(
		double SourceScale,
		const TArray<FTransform>& SourcePose,
		const TArray<FTransform>& TargetPose,
		int32 PairOffset,
		TConstArrayView<FVector3f> FramePointView,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);

	void ApplyAverageBodyPairTargets(
		TArray<FTransform>& OutTargetPose,
		FIKRetargetProcessor& InProcessor,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);
		
	void UpdateTargetPelvisBone(
		TArray<FTransform>& OutTargetPose,
		int32 TargetBoneIdx,
		FIKRetargetProcessor& InProcessor,
		const FVector& TargetWeight,
		const FVector& TargetGoalOffset);

	FVector ComputeTargetWeightedOffset(
		FVector& OutTargetOffset,
		const TArray<int32>& EffectIndices,
		const FVector& BoneLoc,
		FDebugRelativeIKDrawInfo& DebugInfo,
		TArray<FDebugRelativeTargetPairSpace>& DebugPairSpaces);

	FName ApplyBodyMap(FName BodyName) const;
	bool HasBoneDelta(FName TargetBoneName) const;
	const FTransform& GetTargetBoneDelta(FName TargetBoneName) const;

	// Internal structure for holding necessary baked->component transform data
	// Baked data is in Source Bone orientation but at body origin
	struct FBodyTransform
	{
		// TODO: Test this with FMatrix computations and see what's better/faster
		FTransform BoneToBody;
		FVector BodyScale;
		FTransform BodyToGlobal;
	};

	const FTransform& OffsetTransformsForSourceBones(FName SourceBoneName) const;
	void ComputeSourceBodyTransform(FBodyTransform& OutTransform, FName SourceBoneName, const FTransform& GlobalTfm, double SourceScale) const;
	void ComputeTargetBodyTransform(FBodyTransform& OutTransform, FName SourceBoneName, const FTransform& GlobalTfm) const;

	static FVector ApplyBodyTransform(const FBodyTransform& Transform, const FVector& LocalPos);
	static FVector InverseBodyTransform(const FBodyTransform& Transform, const FVector& GlobalPos);
	
	TConstArrayView<FVector3f> ApplyTemporalSmoothing(float Time, float SampleRate, int32 NumSamples, int32 NumBodies, const TArray<FVector3f>& PairLocalVerts);
	
	static FVector CalcReferenceShapeScale3D(FKShapeElem* ShapeElem);
	// Get local body rotation relative to parent bone
	static FQuat GetBodyRotation(const UPhysicsAsset* PhysAsset, FName BoneName);
	// Get body origin relative to parent bone
	static FVector GetBodyTranslation(const UPhysicsAsset* PhysAsset, FName BoneName);
	// Get body oriented scale vector
	static FVector GetBodyOrientedScale(const UPhysicsAsset* PhysAsset, FName BoneName);
	static FKShapeElem* FindBodyShape(const UPhysicsAsset* PhysAsset, FName BoneName);
	
	static double GetDistanceWeight(EDistanceWeightMode DistanceWeightMode, double Distance, double DistThreshold, double DistHalfPointLambda = 0.5);

#if WITH_EDITOR
public:
	UE_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;

	virtual bool HasDebugDrawing() override { return true; };

private:
	void DebugDrawBodyPairs(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugDrawBodyPair>& BodyPairs) const;
	void DebugDrawGoalContributions(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugDrawTargetGoal>& GoalInfoLis) const;
	void DebugDrawPairVertRetarget(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugRelativeTargetPairSpace>& RetargetPairList) const;
	void DebugDrawBodies(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, UPhysicsAsset* PhysAsset, const TArray<FDebugDrawBodyInfo>& PhysBodies, bool TargetPAForSource) const;
	void DebugDrawPhysBody(FPrimitiveDrawInterface* InPDI, const FTransform& ParentTransform, double Scale, const FKShapeElem* ShapeElem, const FLinearColor& Color) const;
	void DebugDrawBodyCoords(FPrimitiveDrawInterface* InPDI, const FTransform& BaseTransform, double Scale, const TArray<FDebugBodyTfmInfo>& BodyTfms) const;

	void UpdateDebugInfo(double SourceScale, const TArray<FTransform>& SourcePose, const TArray<FTransform>& TargetPose, const FDebugRelativeIKDrawInfo& LocalDebugInfo);
	void ResetDebugInfo();

	// Collection of all debug structures
	FDebugRelativeIKDrawInfo DebugDrawInfo;
	
	static UE_API FCriticalSection DebugDataMutex;
#endif
private:
	
	FName SourcePelvicBone;
	// TObjectPtr<const UPhysicsAsset> SourcePhysicsAsset;
	TObjectPtr<const UPhysicsAsset> TargetPhysicsAsset;
	
	// For runtime smoothing
	TArray<FVector3f> SmoothedPoints;

	// Cache notify-related data for faster weighted goal computations
	TObjectPtr<const UAnimSequence> CacheSourceAnimSequence;
	// TObjectPtr<const URelativeBodyBakeAnimNotify> CachedNotifyInfo;
	TObjectPtr<const URelativePropsBakeAnimNotify> CachedPropNotifyInfo;
	int32 CacheMeshPairOffset = 0;
	int32 CachePropPairOffset = 0;
	float AnimSeqPlayHead;
	float MeshSampleRate;
	float PropSampleRate;

	// Cache source/target bone-names
	TArray<FName> SourceBoneNames;
	TArray<FName> TargetBoneNames;

	// Array of body-vert indices affecting each joint goal (for weighted averaging)
	TArray<TArray<int32>> CacheSourceBodyEffectVertIdx;
	TArray<FName> CacheSourceEffectBones;
	TSet<FName> CacheBodyBones;
	TArray<FName> CachePairEffectBoneNames;
	
	TArray<FRelativeIKFrameBoneConstraint> FrameBoneConstraints;

	// Cache skel and IK goal info
	TArray<FTransform> CacheSourceBoneInitTfm;
	TArray<FTransform> CacheTargetBoneInitTfm;
	TMap<FName,FTransform> CacheMapBoneSourceTargetTfm;
	TMap<FName,int32> CacheSourceSkelIndices;
	TMap<FName,int32> CacheTargetSkelIndices;
	// Additional per-frame ik cache maps for length lookup
	TMap<FName,double> CacheChainLengthMap;
	TMap<FName,FVector> CacheChainStartMap;
	// Cache prop bones to pin/update
	TArray<FName> CachePinSourcePropBones;
	TArray<FRelativeIKPinBoneData> CachePinBoneData;
	TArray<FTransform> CachePrevPinTargetPropTransforms;

	// Cached montage instance/segment info for early-out matching in preupdate
	const FAnimMontageInstance* MontageInstance;
	float SegmentStartTime;
	float SegmentEndTime;
};

/* The blueprint/python API for editing a Relative Pelvic Motion Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetPelvicMotionController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current op settings as a struct.
	 * @return FIKRetargetRelativeIKOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetPelvicMotionOpSettings GetSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRetargetRelativeIKOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetPelvicMotionOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
